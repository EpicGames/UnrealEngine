{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "File Sandbox",
	"Description": "Core functionality for sandboxing files in the editor.",
	"Category": "Developer",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "FileSandboxCore",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "FileSandboxUI",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}