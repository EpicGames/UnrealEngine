// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class FileSandboxCore : ModuleRules
	{
		public FileSandboxCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",  
				}
			);
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"FileUtilities",
					"Json",
					"JsonUtilities", 
					"RenderCore",
					"SourceControl", 
				});

			if (Target.bUsesSlate)
			{
				PrivateDependencyModuleNames.AddRange(new string[]
				{
					"SlateCore" // Needed for source control proxy returning SNullWidget::NullWidget
				});
			}

			if (Target.bBuildEditor)
			{
				// DirectoryWatcher module is a disallowed dependency for some internal tools that use a runtime version of this build. 
				PrivateDefinitions.Add("UE_SANDBOX_WITH_DIRECTORY_WATCHER=1");
				PrivateDependencyModuleNames.Add("DirectoryWatcher");
				
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"AssetRegistry",
						"EngineSettings",
						"UnrealEd"
					}
				);
			}
			else
			{
				PrivateDefinitions.Add("UE_SANDBOX_WITH_DIRECTORY_WATCHER=0");
			}
		}
	}
}
