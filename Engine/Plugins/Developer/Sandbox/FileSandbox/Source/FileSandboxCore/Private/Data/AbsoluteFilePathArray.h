// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Paths.h"
#include "AbsoluteFilePathArray.generated.h"

/** Keeps an array of absolute file paths. Each path is unique. */
USTRUCT()
struct FFileSandboxCore_AbsoluteFilePathArray
{
	GENERATED_BODY()

	const TArray<FString>& GetFiles() const { return Files; }

	/** @return Whether the path was added, i.e. was not already present. */
	bool Add(const FString& InPath)
	{
		const int32 NumBefore = Files.Num();
		Files.AddUnique(FPaths::ConvertRelativePathToFull(InPath));
		return NumBefore != Files.Num();
	}

	/** @return Whether anything was removed. */
	bool Remove(const FString& InPath)
	{
		const int32 NumRemoved = Files.RemoveSingle(FPaths::ConvertRelativePathToFull(InPath));
		return NumRemoved > 0;
	}

	void Empty()
	{
		Files.Empty();
	}

private:

	/** Absolute file paths. */
	UPROPERTY()
	TArray<FString> Files;
};