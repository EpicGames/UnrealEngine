// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AbsoluteFilePathArray.h"
#include "MountPointInfo.h"
#include "TimedAbsoluteFilePathArray.h"
#include "Data/VersionInfo.h"
#include "ManifestData.generated.h"

/** 
 * Internal bookkeeping data for the sandbox logic.
 * This data is owned by the sandbox system and should not be directly modified via public API.
 */
USTRUCT()
struct FFileSandboxCore_ManifestData
{
	GENERATED_BODY()
	
	/**
	 * Absolute non-sandbox paths to files deleted in sandbox. 
	 * Also saved the timestamp of the remove operation.
	 */
	UPROPERTY()
	FFileSandboxCore_TimedAbsoluteFilePathArray DeletedFiles;

	/** Absolute non-sandbox paths to files modified in sandbox. */
	UPROPERTY()
	FFileSandboxCore_AbsoluteFilePathArray ModifiedFiles;

	/** Absolute non-sandbox paths to files added in sandbox. */
	UPROPERTY()
	FFileSandboxCore_AbsoluteFilePathArray AddedFiles;
	
	/**
	 * Version info when this sandbox was created.
	 * Holds info about the file version, engine version, and custom versions (for FArchives).
	 * 
	 * This information is saved in the manifest to minimize the risk of API users directly editing and corrupting it.
	 * It could also have been saved in the metadata file but that has API for mutating it (= risk of corruption / mutation via API).
	 * 
	 * May be needed to parse the content in FFileSandboxCore_SandboxMetaData. 
	 * TODO UE-350242: This information can also be used to reject loading a sandbox with newer version info than the current engine version.
	 */
	UPROPERTY()
	FFileSandboxCore_VersionInfo VersionInfo;

	/**
	 * The mount points that were referenced by either DeletedFiles, ModifiedFiles, or AddedFiles when the manifest was last saved.
	 * This information is used to migrate file paths when a sandbox instance is created in case a mount point location changes, e.g. when importing
	 * a sandbox or if the user changes the location of the underlying uproject.
	 */
	UPROPERTY()
	FFileSandboxCore_MountPointsInfo MountPoints;

	void Empty()
	{
		DeletedFiles.Empty();
		ModifiedFiles.Empty();
		AddedFiles.Empty();
	}
	
	void Initialize()
	{
		VersionInfo.Initialize();
		MountPoints.Initialize();
	}
};