// Copyright Epic Games, Inc. All Rights Reserved.

#include "MountPointInfo.h"

#include "Misc/PackageName.h"
#include "Misc/Paths.h"

void FFileSandboxCore_MountPointsInfo::Initialize()
{
	check(MountPoints.IsEmpty());
	
	TArray<FString> RootContentPaths;
	FPackageName::QueryRootContentPaths(RootContentPaths);
	
	MountPoints.Reserve(RootContentPaths.Num());
	for (FString& MountPointName : RootContentPaths)
	{
		FString FilesystemPath = FPackageName::LongPackageNameToFilename(MountPointName);
		MountPoints.Emplace(
			MountPointName,
			FPaths::ConvertRelativePathToFull(FilesystemPath)
			);
	}
}
