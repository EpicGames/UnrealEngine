// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "MountPointInfo.generated.h"

/** Describes a single mount point. */
USTRUCT()
struct FFileSandboxCore_MountPoint
{
	GENERATED_BODY()
	
	/** Logical name of the mount point, e.g. "/Game/" */
	UPROPERTY()
	FString MountName;
	
	/** Absolute path to the root of the mount point. e.g. "C:/MyProject/Content" */
	UPROPERTY()
	FString RootPath;
	
	FFileSandboxCore_MountPoint() = default;
	FFileSandboxCore_MountPoint(FString InMountName, FString InRootPath) : MountName(MoveTemp(InMountName)), RootPath(MoveTemp(InRootPath)) {}
};

/** Holds mount points the engine has. */
USTRUCT()
struct FFileSandboxCore_MountPointsInfo
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FFileSandboxCore_MountPoint> MountPoints;
	
	/** Adds all mount points currently active to MountPoints. */
	void Initialize();
};