// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Paths.h"
#include "Misc/DateTime.h"
#include "TimedAbsoluteFilePathArray.generated.h"

/** Keeps an array of absolute file paths and their respective timestamps. Each path is unique. */
USTRUCT()
struct FFileSandboxCore_TimedAbsoluteFilePathArray
{
	GENERATED_BODY()

	const TArray<FString>& GetFiles() const { return Files; }
	
	const TArray<FDateTime>& GetTimestamps() const { return Timestamps; }
	bool AreTimestampsValid() const { return Files.Num() == Timestamps.Num(); }

	/** @return The timestamp of the file, or unset. */
	TOptional<FDateTime> GetTimestamp(const FString& InPath) const
	{
		const FString Path = FPaths::ConvertRelativePathToFull(InPath);
		const int32 Index = Files.IndexOfByKey(Path);
		return Index != INDEX_NONE ? Timestamps[Index] : TOptional<FDateTime>();
	}
	
	/** @return Whether the path was added, i.e. was not already present. */
	bool Add(const FString& InPath, const FDateTime& InTimestamp = FDateTime::UtcNow())
	{
		const int32 NumBefore = Files.Num();
		const FString Path = FPaths::ConvertRelativePathToFull(InPath);
		
		if (!Files.Contains(Path))
		{
			Files.Add(Path);
			Timestamps.Add(InTimestamp);
		}
		
		return NumBefore != Files.Num();
	}

	/** @return Whether anything was removed. */
	bool Remove(const FString& InPath)
	{
		const int32 Index = Files.IndexOfByKey(FPaths::ConvertRelativePathToFull(InPath));
		
		const bool bRemove = Files.IsValidIndex(Index);
		if (bRemove)
		{
			Files.RemoveAt(Index);
		}
		
		// User may have edited the manifest file and corrupted it
		if (bRemove && ensure(Timestamps.IsValidIndex(Index)))
		{
			Timestamps.RemoveAt(Index);
		}
		
		return bRemove;
	}

	void Empty()
	{
		Files.Empty();
		Timestamps.Empty();
	}

private:

	/** Absolute file paths. */
	UPROPERTY()
	TArray<FString> Files;
	
	/**
	 * The times that the files were changed in local time. 
	 * 
	 * Under normal circumstances, this is equal length to Files.
	 * Keep in mind that the user may have edited the file in the manifest, in which case the values in this array are useless.
	 */
	UPROPERTY()
	TArray<FDateTime> Timestamps;
};
