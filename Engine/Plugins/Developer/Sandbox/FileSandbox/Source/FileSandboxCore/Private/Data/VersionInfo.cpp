// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/VersionInfo.h"

#include "Containers/UnrealString.h"
#include "Misc/EngineVersion.h"
#include "Serialization/CustomVersion.h"

void FFileSandboxCore_FileVersionInfo::Initialize()
{
	FileVersionUE5 = GPackageFileUEVersion.FileVersionUE5;
	FileVersionLicensee = GPackageFileLicenseeUEVersion;
}

FString FFileSandboxCore_FileVersionInfo::ToString() const
{
	return FString::Printf(TEXT("FileVersionUE5=%d Licensee=%d"), FileVersionUE5, FileVersionLicensee);
}

void FFileSandboxCore_EngineVersionInfo::Initialize(const FEngineVersion& InVersion)
{
	Major = InVersion.GetMajor();
	Minor = InVersion.GetMinor();
	Patch = InVersion.GetPatch();
	Changelist = InVersion.GetChangelist();
}

FString FFileSandboxCore_EngineVersionInfo::ToString(bool bIncludeChangelist) const
{
	return bIncludeChangelist
		? FString::Printf(TEXT("%u.%u.%u-%u"), Major, Minor, Patch, Changelist)
		: FString::Printf(TEXT("%u.%u.%u"), Major, Minor, Patch);
}

bool FFileSandboxCore_EngineVersionInfo::IsCompatibleWith(const FFileSandboxCore_EngineVersionInfo& InCurrentVersion) const
{
	// Check major version
	if (Major > InCurrentVersion.Major)
	{
		return false;
	}
	if (Major < InCurrentVersion.Major)
	{
		return true;
	}

	// Major versions match, check minor
	if (Minor > InCurrentVersion.Minor)
	{
		return false;
	}
	if (Minor < InCurrentVersion.Minor)
	{
		return true;
	}

	// Major and minor match, check patch
	if (Patch > InCurrentVersion.Patch)
	{
		return false;
	}
	if (Patch < InCurrentVersion.Patch)
	{
		return true;
	}

	// Major, minor, and patch match - check changelist
	// A sandbox with a higher changelist cannot be loaded in a lower changelist build
	return Changelist <= InCurrentVersion.Changelist;
}

void FFileSandboxCore_CustomVersionInfo::Initialize(const FCustomVersion& InVersion)
{
	FriendlyName = InVersion.GetFriendlyName();
	Key = InVersion.Key;
	Version = InVersion.Version;
}

void FFileSandboxCore_VersionInfo::Initialize()
{
	FileVersion.Initialize();
	EngineVersion.Initialize(FEngineVersion::Current());

	CustomVersions.Empty(CustomVersions.Num());
	FCustomVersionContainer AllCurrentVersions = FCurrentCustomVersions::GetAll();
	for (const FCustomVersion& EngineCustomVersion : AllCurrentVersions.GetAllVersions())
	{
		FFileSandboxCore_CustomVersionInfo& CustomVersion = CustomVersions.AddDefaulted_GetRef();
		CustomVersion.Initialize(EngineCustomVersion);
	}
}

bool FFileSandboxCore_VersionInfo::IsInitialized() const
{
	return CustomVersions.Num() > 0;
}

FString FFileSandboxCore_VersionInfo::ToString() const
{
	return FString::Printf(TEXT("%s %s"), *EngineVersion.ToString(), *FileVersion.ToString());
}
