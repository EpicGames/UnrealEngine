// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileSandboxCoreModule.h"

#include "Misc/CoreDelegates.h"
#include "Repository/NaiveSandboxRepository.h"
#include "Utils/CommandLineUtils.h"
#include "Utils/SandboxDirectoryUtils.h"

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
#include "Repository/WatchedSandboxRepository.h"
#endif

namespace UE::FileSandboxCore
{
void FFileSandboxCoreModule::StartupModule()
{
	SandboxManager = MakeUnique<FSandboxManager>();
	
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
	DefaultRepository = MakeUnique<FWatchedSandboxRepository>(GetBaseSandboxDirectory(), *SandboxManager);
#else
	DefaultRepository = MakeUnique<FNaiveSandboxRepository>(GetBaseSandboxDirectory(), *SandboxManager);
#endif
	
	RegisterStartupCommandLineDelegate();
}

void FFileSandboxCoreModule::ShutdownModule()
{
	DefaultRepository.Reset();
	SandboxManager.Reset();
}
}

IMPLEMENT_MODULE(UE::FileSandboxCore::FFileSandboxCoreModule, FileSandboxCore);

