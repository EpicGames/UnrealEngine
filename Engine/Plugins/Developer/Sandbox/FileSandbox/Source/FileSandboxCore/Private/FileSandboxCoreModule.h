// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IFileSandboxCoreModule.h"
#include "Sandbox/SandboxManager.h"

namespace UE::FileSandboxCore
{
class FFileSandboxCoreModule : public IFileSandboxCoreModule
{
public:

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	//~ Begin IFileSandboxCoreModule Module
	virtual ISandboxManager& GetSandboxManager() override { return *SandboxManager; }
	virtual ISandboxRepository& GetDefaultSandboxRepository() override { return *DefaultRepository; }
	//~ End IFileSandboxCoreModule Module

private:

	/** Public API for interacting with the sandbox instance. */
	TUniquePtr<FSandboxManager> SandboxManager;
	
	/** Caches the sandboxes in the default sandbox directory. */
	TUniquePtr<ISandboxRepository> DefaultRepository;
};
}

