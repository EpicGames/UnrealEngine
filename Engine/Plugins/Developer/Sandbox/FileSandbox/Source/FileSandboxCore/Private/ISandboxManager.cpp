// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISandboxManager.h"

#include "Interface/ISandboxLock.h"
#include "Misc/DateTime.h"
#include "Types/EBreakBehavior.h"
#include "Types/Manager/DeleteSandboxByDirectoryArgs.h"
#include "Types/Manager/DeleteSandboxByNameArgs.h"
#include "Types/Manager/DeleteSandboxResult.h"
#include "Types/Manager/LoadSandboxByDirectoryArgs.h"
#include "Types/Manager/LoadSandboxByNameArgs.h"
#include "Types/Manager/LoadSandboxError.h"
#include "Types/Manager/SandboxCreationResult.h"
#include "Utils/PackageSandboxUtils.h"
#include "Utils/SandboxDirectoryUtils.h"

namespace UE::FileSandboxCore
{
FLoadSandboxResult ISandboxManager::LoadNamedSandbox(const FLoadSandboxByNameArgs& InArgs)
{
	TOptional<FString> FullPath = FindSandboxByName(InArgs.Name, InArgs.BaseDirectory);
	if (!FullPath)
	{
		return { MakeError<FLoadSandboxError>(FLoadSandboxError{ ELoadSandboxLoadErrorReason::InvalidName }) };
	}
	
	FLoadSandboxByDirectoryArgs LoadByDirArgs(MoveTemp(*FullPath));
	LoadByDirArgs.InitArgs = InArgs.InitArgs;
	return LoadSandbox(LoadByDirArgs);
}

FDeleteSandboxResult ISandboxManager::DeleteNamedSandbox(const FDeleteSandboxByNameArgs& InArgs)
{
	TOptional<FString> FullPath = FindSandboxByName(InArgs.Name, InArgs.BaseDirectory);
	if (!FullPath)
	{
		return FDeleteSandboxResult(EDeleteSandboxErrorCode::InvalidName);
	}
	
	return DeleteSandbox(MoveTemp(*FullPath));
}

ELeaveSandboxErrorCode ISandboxManager::CanLeaveSandboxWithReason() const
{
	ISandboxLock* ActiveLock = GetActiveLock();
	const bool bCanLeaveDueToLock = !ActiveLock || ActiveLock->CanLeaveSandbox();
	if (!bCanLeaveDueToLock)
	{
		return ELeaveSandboxErrorCode::Locked;
	}
	
	return ELeaveSandboxErrorCode::Success;
}

void ISandboxManager::EnumerateFileChangesByName(
	const FString& InSandboxName, TFunctionRef<FProcessFileChangeSignature> InProcess, const FString& InBaseDirectory
	) const
{
	if (const TOptional<FString> FullPath = FindSandboxByName(InSandboxName, InBaseDirectory))
	{
		EnumerateFileChanges(*FullPath, InProcess);
	}
}
}
