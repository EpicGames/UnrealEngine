// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interface/DefaultPackageReloadHandler.h"

#include "Types/Package/HotReloadPackageArgs.h"
#include "Types/Package/PurgePackageArgs.h"
#include "Utils/PackageSandboxUtils.h"

namespace UE::FileSandboxCore
{
void FDefaultPackageReloadHandler::PurgePackages(const FPurgePackageArgs& InArgs)
{
	FileSandboxCore::PurgePackages(InArgs.PackageNames);
}

void FDefaultPackageReloadHandler::HotReloadPackages(const FHotReloadPackageArgs& InArgs)
{
	FileSandboxCore::HotReloadPackages(InArgs.PackageNames);
}
}
