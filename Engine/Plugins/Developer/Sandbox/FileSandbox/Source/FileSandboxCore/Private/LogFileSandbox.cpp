// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogFileSandbox.h"

DEFINE_LOG_CATEGORY(LogFileSandbox);