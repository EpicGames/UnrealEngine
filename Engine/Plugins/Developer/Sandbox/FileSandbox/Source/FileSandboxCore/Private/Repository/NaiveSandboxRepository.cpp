// Copyright Epic Games, Inc. All Rights Reserved.

#include "NaiveSandboxRepository.h"

#include "Data/VersionInfo.h"
#include "ISandboxManager.h"
#include "Types/EBreakBehavior.h"
#include "Types/RepositoryChangedEvent.h"
#include "Types/SandboxMetaInfo.h"
#include "Utils/SandboxDirectoryUtils.h"

namespace UE::FileSandboxCore
{
FNaiveSandboxRepository::FNaiveSandboxRepository(FString InBaseDirectory, ISandboxManager& InManager)
	: BaseDirectory(MoveTemp(InBaseDirectory))
	, Manager(InManager)
{
	Manager.OnPostSandboxStartup().AddRaw(this, &FNaiveSandboxRepository::OnSandboxStartup);
}

FNaiveSandboxRepository::~FNaiveSandboxRepository()
{
	Manager.OnPostSandboxStartup().RemoveAll(this);
}

void FNaiveSandboxRepository::ForEachSandbox(TFunctionRef<EBreakBehavior(const FString& InRootPath, const FSandboxMetaInfo&)> InProcessSandboxes)
{
	FileSandboxCore::ForEachSandbox([this, &InProcessSandboxes](const FString& InRootPath)
	{
		EBreakBehavior Behavior = EBreakBehavior::Continue;
		ReadMetaData(InRootPath, [&InRootPath, &InProcessSandboxes, &Behavior](const FSandboxMetaInfo& InMetaData)
		{
			Behavior = InProcessSandboxes(InRootPath, InMetaData);
		});
		return Behavior;
	}
	, BaseDirectory);
}

int32 FNaiveSandboxRepository::NumSandboxes() const
{
	int32 Num = 0;
	FileSandboxCore::ForEachSandbox([&Num](auto){ ++Num; return EBreakBehavior::Continue; }, BaseDirectory);
	return Num;
}

bool FNaiveSandboxRepository::ReadMetaData(const FString& InRootPath, TFunctionRef<void(const FSandboxMetaInfo&)> InProcessMetadata)
{
	const TOptional<FFileSandboxCore_SandboxMetaData> MetaData = LoadMetaData(InRootPath);
	const FDateTime DateTime = GetLastModified(InRootPath);
	const FFileSandboxCore_VersionInfo VersionInfo = LoadVersionInfo(InRootPath);
	if (MetaData && DateTime != FDateTime::MinValue() && VersionInfo.IsInitialized())
	{
		InProcessMetadata(FSandboxMetaInfo(*MetaData, DateTime, VersionInfo));
		return true;
	}
	return false;
}

void FNaiveSandboxRepository::OnSandboxStartup(ISandboxInstance& SandboxInstance) const
{
	const FString RootDir(SandboxInstance.GetRootDirectory());
	OnSandboxesChangedDelegate.Broadcast(FRepositoryChangedEvent{ .AddedSandboxPaths = MakeArrayView(&RootDir, 1) });
}
}
