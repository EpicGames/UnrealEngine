// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/SandboxMetaData.h"
#include "ISandboxInstance.h"
#include "ISandboxRepository.h"
#include "Templates/UnrealTemplate.h"

struct FFileChangeData;

namespace UE::FileSandboxCore
{
class ISandboxManager;

/** 
 * Naive implementation that always reads the disk and does no caching. Never invokes OnSandboxMetaDataChanged. 
 * Used when directory watching is not possible.
 */
class FNaiveSandboxRepository : public ISandboxRepository, public FNoncopyable
{
public:
	
	explicit FNaiveSandboxRepository(FString InBaseDirectory, ISandboxManager& InManager UE_LIFETIMEBOUND);
	~FNaiveSandboxRepository();

	//~ Begin ISandboxRepository Interface
	virtual void ForEachSandbox(TFunctionRef<EBreakBehavior(const FString& InRootPath, const FSandboxMetaInfo&)> InProcessSandboxes) override;
	virtual int32 NumSandboxes() const override;
	virtual bool ReadMetaData(const FString& InRootPath, TFunctionRef<void(const FSandboxMetaInfo&)> InProcessMetadata) override;
	virtual FOnSandboxesChanged& OnSandboxesChanged() override { return OnSandboxesChangedDelegate; }
	virtual FOnSandboxMetaDataChanged& OnSandboxMetaDataChanged() override { return OnSandboxMetaDataChangedDelegate; }
	//~ End ISandboxRepository Interface

private:
	
	/** The base directory in which to look for sandboxes. Only looks at the top-level, i.e. not recursively. */
	const FString BaseDirectory;
	
	/** Sandbox manager. Used to detect when a sandbox is created. */
	ISandboxManager& Manager;
	
	/** Invoked when the known sandboxes changes (removed, discovered, etc.) */
	FOnSandboxesChanged OnSandboxesChangedDelegate;
	/** Invoked when a sandbox's metadata changes. */
	FOnSandboxMetaDataChanged OnSandboxMetaDataChangedDelegate;
	
	/** Callback when a sandbox is started. */
	void OnSandboxStartup(ISandboxInstance& SandboxInstance) const;
};	
}