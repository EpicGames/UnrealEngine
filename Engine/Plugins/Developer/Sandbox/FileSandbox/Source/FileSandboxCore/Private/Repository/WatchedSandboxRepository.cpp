// Copyright Epic Games, Inc. All Rights Reserved.

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
#include "WatchedSandboxRepository.h"

#include "Algo/AnyOf.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "IDirectoryWatcher.h"
#include "ISandboxInstance.h"
#include "ISandboxManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Types/EBreakBehavior.h"
#include "Types/RepositoryChangedEvent.h"
#include "Utils/SandboxDirectoryUtils.h"
#include "Utils/SandboxFileUtils.h"
#include "Utils/Watcher/DirectoryWatcherUtils.h"

namespace UE::FileSandboxCore
{
FWatchedSandboxRepository::FWatchedSandboxRepository(FString InBaseDirectory, ISandboxManager& InSandboxManager)
	: BaseDirectory(MoveTemp(InBaseDirectory))
	, SandboxManager(InSandboxManager)
	, WatchedBaseDirectory([this]()
	{
		IDirectoryWatcher* DirectoryWatcher = GetDirectoryWatcher();
		if (!DirectoryWatcher)
		{
			return FScopedWatchedDirectory{};;
		}
			
		// Would be etter to wait until a user action causes directory to be created. 
		// However, it's easier implementation-wise to always create this folder.
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		const bool bDirectoryExistedAlready = PlatformFile.DirectoryExists(*BaseDirectory);
		const bool bHasCreatedDirectory = !bDirectoryExistedAlready && PlatformFile.CreateDirectoryTree(*BaseDirectory);
			
		const bool bCanSetupWatcher = bHasCreatedDirectory || bDirectoryExistedAlready;
		if (bCanSetupWatcher)
		{
			FDelegateHandle Handle;
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
				BaseDirectory, 
				IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FWatchedSandboxRepository::OnBaseDirectoryChanged),
				Handle,
				// We only care for the direct directories. Ignore stuff in the sandbox sub-directories.
				IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges | IDirectoryWatcher::WatchOptions::IgnoreChangesInSubtree
				);
			return FScopedWatchedDirectory(BaseDirectory, Handle);
		}
			
		return FScopedWatchedDirectory{};
	}())
{
	RescanBaseDirectory();
	UpdateWatchedDirectories();
	
	SandboxManager.OnPostSandboxStartup().AddRaw(this, &FWatchedSandboxRepository::OnSandboxStartup);
	FCoreDelegates::OnEndFrame.AddRaw(this, &FWatchedSandboxRepository::OnEndOfFrame);
}

FWatchedSandboxRepository::~FWatchedSandboxRepository()
{
	SandboxManager.OnPostSandboxStartup().RemoveAll(this);
	FCoreDelegates::OnEndFrame.RemoveAll(this);
}
	
void FWatchedSandboxRepository::ForEachSandbox(TFunctionRef<EBreakBehavior(const FString& InRootPath, const FSandboxMetaInfo& MetaData)> InProcessSandboxes)
{
	for (const FSandboxData& SandboxData : CachedSandboxData)
	{
		if (InProcessSandboxes(SandboxData.AbsPath, SandboxData.CachedData) == EBreakBehavior::Break)
		{
			break;
		}
	}
}

bool FWatchedSandboxRepository::ReadMetaData(const FString& InRootPath, TFunctionRef<void(const FSandboxMetaInfo&)> InProcessMetadata)
{
	const int32 Index = IndexOf(InRootPath);
	if (CachedSandboxData.IsValidIndex(Index))
	{
		InProcessMetadata(CachedSandboxData[Index].CachedData);
		return true;
	}
	return false;
}

void FWatchedSandboxRepository::RescanBaseDirectory()
{
	CachedSandboxData.Reset();
	
	FileSandboxCore::ForEachSandbox([this](const FString& InBaseDirectory)
	{
		AddSandboxIfExists(InBaseDirectory);
		return EBreakBehavior::Continue;
	}, BaseDirectory);
}

void FWatchedSandboxRepository::AddSandboxIfExists(const FString& InDirectory)
{
	if (!IsRootSandboxDirectory(InDirectory))
	{
		return;
	}
	
	const FString AbsPath = FPaths::ConvertRelativePathToFull(InDirectory);
	TOptional<FFileSandboxCore_SandboxMetaData> MetaData = LoadMetaData(AbsPath);
	FDateTime LastModified = GetLastModified(AbsPath);
	FFileSandboxCore_VersionInfo VersionInfo = LoadVersionInfo(AbsPath);
	if (!MetaData || LastModified == FDateTime::MinValue() || !VersionInfo.IsInitialized())
	{
		return;
	}
	
	const int32 ExistingIndex = IndexOf(AbsPath);
	FSandboxMetaInfo MetaInfo(MoveTemp(*MetaData), LastModified, MoveTemp(VersionInfo));
	if (CachedSandboxData.IsValidIndex(ExistingIndex)) // This is just defensive code - this case should not occur.
	{
		CachedSandboxData[ExistingIndex].CachedData = MoveTemp(MetaInfo);
		OnSandboxMetaDataChangedDelegate.Broadcast(AbsPath);
	}
	else
	{
		CachedSandboxData.Emplace(FSandboxData{ AbsPath, MoveTemp(MetaInfo) });
		OnSandboxesChanged().Broadcast(FRepositoryChangedEvent{ .AddedSandboxPaths = MakeArrayView(&AbsPath, 1) });
	}
}

int32 FWatchedSandboxRepository::IndexOf(const FString& InDirectory) const
{
	return CachedSandboxData.IndexOfByPredicate([&InDirectory](const FSandboxData& Data)
	{
		return FPaths::IsSamePath(Data.AbsPath, InDirectory);
	});
}

void FWatchedSandboxRepository::OnBaseDirectoryChanged(const TArray<FFileChangeData>& InChanges)
{
	check(IsInGameThread());
	
	UpdateCacheFromChangedBaseDirectory(InChanges);
	UpdateWatchedDirectories();
}

void FWatchedSandboxRepository::OnSandboxDirectoryChanged(const TArray<FFileChangeData>& InChanges, FString InSandboxPath)
{
	check(IsInGameThread());
	
	const int32 Index = IndexOf(InSandboxPath);
	
	const bool bWasSandboxDirectoryBefore = CachedSandboxData.IsValidIndex(Index);
	const bool bIsSandboxDirectory = IsRootSandboxDirectory(InSandboxPath);
	if (bIsSandboxDirectory && !bWasSandboxDirectoryBefore)
	{
		AddSandboxIfExists(InSandboxPath);
		return;
	}
	if (!bWasSandboxDirectoryBefore)
	{
		return;
	}
	
	if (!bIsSandboxDirectory)
	{
		CachedSandboxData.RemoveAt(Index);
		OnSandboxesChanged().Broadcast(FRepositoryChangedEvent{ .RemovedSandboxPaths = MakeArrayView(&InSandboxPath, 1) });
		return;
	}
	
	bool bChangedMetaData = false;
	const auto SandboxNotRemoved = [this, &InSandboxPath, Index] { return CachedSandboxData.IsValidIndex(Index) && CachedSandboxData[Index].AbsPath == InSandboxPath; };
	bChangedMetaData |= DetectMetaDataFileChange(Index, InChanges, InSandboxPath);
	bChangedMetaData |= SandboxNotRemoved() && DetectManifestFileChange(Index, InChanges, InSandboxPath);
	if (bChangedMetaData)
	{
		OnSandboxMetaDataChangedDelegate.Broadcast(InSandboxPath);
	}
}

bool FWatchedSandboxRepository::DetectMetaDataFileChange(int32 InSandboxIndex, const TArray<FFileChangeData>& InChanges, const FString& InSandboxPath)
{
	const bool bContainsUnknownOrRescan = InChanges.ContainsByPredicate([](const FFileChangeData& Change)
	{
		return Change.Action == FFileChangeData::FCA_RescanRequired || Change.Action == FFileChangeData::FCA_Unknown;
	});
	const int32 MetadataFileChangeIndex = InChanges.IndexOfByPredicate([](const FFileChangeData& Change)
	{
		return FPaths::GetCleanFilename(Change.Filename) == GetMetadataFileName();
	});
	// If the directory watcher is unsure what happened (rescan / unknown), we'll treat the file as changed, i.e. parse it again.
	if (!bContainsUnknownOrRescan && !InChanges.IsValidIndex(MetadataFileChangeIndex))
	{
		return false;
	}
	
	if (TOptional<FFileSandboxCore_SandboxMetaData> MetaData = LoadMetaData(InSandboxPath))
	{
		CachedSandboxData[InSandboxIndex].CachedData.UserMetaData = MoveTemp(*MetaData);
		return true;
	}
	else
	{
		// Failed to load metadata. We'll treat it as the metadata file being corrupted or no longer existing.
		CachedSandboxData.RemoveAtSwap(InSandboxIndex);
		OnSandboxesChangedDelegate.Broadcast(FRepositoryChangedEvent{ .RemovedSandboxPaths = MakeArrayView(&InSandboxPath, 1) });
		return false;
	}
}

bool FWatchedSandboxRepository::DetectManifestFileChange(int32 InSandboxIndex, const TArray<FFileChangeData>& InChanges, const FString& InSandboxPath)
{
	const bool bContainsUnknownOrRescan = InChanges.ContainsByPredicate([](const FFileChangeData& Change)
	{
		return Change.Action == FFileChangeData::FCA_RescanRequired || Change.Action == FFileChangeData::FCA_Unknown;
	});
	const int32 ManifestFileChangeIndex = InChanges.IndexOfByPredicate([](const FFileChangeData& Change)
	{
		return FPaths::GetCleanFilename(Change.Filename) == GetManifestFileName();
	});
	
	if (!bContainsUnknownOrRescan && !InChanges.IsValidIndex(ManifestFileChangeIndex))
	{
		return false;
	}
	
	const FDateTime LastModified = GetLastModified(InSandboxPath);
	if (LastModified != FDateTime::MinValue())
	{
		CachedSandboxData[InSandboxIndex].CachedData.LastModified = LastModified;
		return true;
	}
	else
	{
		// Failed to obtain info about timestamp. We'll treat it as the file no longer existing.
		CachedSandboxData.RemoveAtSwap(InSandboxIndex);
		OnSandboxesChangedDelegate.Broadcast(FRepositoryChangedEvent{ .RemovedSandboxPaths = MakeArrayView(&InSandboxPath, 1) });
		return false;
	}
}

void FWatchedSandboxRepository::OnEndOfFrame()
{
	PendingDirectoryWatchersToUnregister.Reset();
}

void FWatchedSandboxRepository::UpdateCacheFromChangedBaseDirectory(const TArray<FFileChangeData>& InChanges)
{
	const bool bDoFullRescan = Algo::AnyOf(InChanges, [](const FFileChangeData& Change)
	{
		const bool bIsDirectoryChange = FPaths::DirectoryExists(Change.Filename);
		return bIsDirectoryChange && (Change.Action == FFileChangeData::FCA_Unknown || Change.Action == FFileChangeData::FCA_RescanRequired);
	});
	if (bDoFullRescan)
	{
		RescanBaseDirectory();
		return;
	}
	
	TSet<FString> AlreadyProcessed;
	// Process in reverse order to prioritize the latest change and ignore earlier changes.
	// E.g. if InChanges is { { Modified, "Foo"}, { Removed, "Foo"} }, we just care about the Removed event.
	for (int32 FileIndex = InChanges.Num() - 1; FileIndex >= 0; --FileIndex)
	{
		const FFileChangeData& Change = InChanges[FileIndex];
		const FString& Path = Change.Filename;
		
		// InChanges can contain 100s of FCA_Modified changes for the same file. Don't do double work.
		if (AlreadyProcessed.Contains(Path))
		{
			continue;
		}
		AlreadyProcessed.Add(Path);

		switch (Change.Action)
		{
		case FFileChangeData::FCA_Added: AddSandboxIfExists(Path); break;
		case FFileChangeData::FCA_Removed: 
			if (const int32 SandboxIndex = IndexOf(Path); CachedSandboxData.IsValidIndex(SandboxIndex))
			{
				CachedSandboxData.RemoveAtSwap(SandboxIndex);
				OnSandboxesChanged().Broadcast(FRepositoryChangedEvent{ .RemovedSandboxPaths = MakeArrayView(&Path, 1) });
			}
			break;
			
		case FFileChangeData::FCA_Modified: break; // Don't care. The sandbox name is what it says in the metadata file, not the directory name.
		default: checkNoEntry();
		}
	}
}

void FWatchedSandboxRepository::UpdateWatchedDirectories()
{
	IDirectoryWatcher* DirectoryWatcher = GetDirectoryWatcher();
	if (!DirectoryWatcher)
	{
		return;
	}
	
	// Reuse old delegates if possible
	TArray<FScopedWatchedDirectory> Old = MoveTemp(WatchedSubDirectories);
	
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.IterateDirectory(*BaseDirectory, [this, &Old, DirectoryWatcher](const TCHAR* FilenameOrDirectory, const bool bIsDirectory)
	{
		if (!bIsDirectory)
		{
			return true;
		}
			
		const int32 ExistingIndex = Old.IndexOfByPredicate([&FilenameOrDirectory](const FScopedWatchedDirectory& WatchedDirectory)
		{
			return FPaths::IsSamePath(WatchedDirectory.GetWatchedDirectory(), FilenameOrDirectory);
		});
			
		if (Old.IsValidIndex(ExistingIndex))
		{
			WatchedSubDirectories.Emplace(MoveTemp(Old[ExistingIndex]));
		}
		else
		{
			FDelegateHandle Handle;
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
				FilenameOrDirectory, 
				IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FWatchedSandboxRepository::OnSandboxDirectoryChanged, FString(FilenameOrDirectory)),
				Handle,
				IDirectoryWatcher::WatchOptions::IgnoreChangesInSubtree // We only care for the manifest and metadata file
				);
			WatchedSubDirectories.Emplace(FilenameOrDirectory, Handle);
		}
			
		return true;
	});
	
	// UpdateWatchedDirectories may be called by DirectoryWatcher to process changes.
	// It's not valid to call UnregisterDirectoryChangedCallback_Handle while the DirectoryWatcher module is processing notifications, so defer.
	for (FScopedWatchedDirectory& WatchedDirectory : Old)
	{
		if (WatchedDirectory.IsValid())
		{
			PendingDirectoryWatchersToUnregister.Emplace(MoveTemp(WatchedDirectory));
		}
	}
}

void FWatchedSandboxRepository::OnSandboxStartup(ISandboxInstance& SandboxInstance)
{
	AddSandboxIfExists(SandboxInstance.GetRootDirectory());
}
}

#endif
