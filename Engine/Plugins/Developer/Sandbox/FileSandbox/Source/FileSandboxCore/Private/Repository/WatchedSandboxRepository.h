// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
#include "Data/SandboxMetaData.h"
#include "ISandboxRepository.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SandboxMetaInfo.h"
#include "Utils/Watcher/ScopedWatchedDirectory.h"

struct FFileChangeData;

namespace UE::FileSandboxCore
{
class ISandboxInstance;
class ISandboxManager;

/** Simple directory watcher that detects when sandboxes are added, removed, and metadata file edited. */
class FWatchedSandboxRepository : public ISandboxRepository, public FNoncopyable
{
public:
	
	explicit FWatchedSandboxRepository(FString InBaseDirectory, ISandboxManager& InSandboxManager UE_LIFETIMEBOUND);
	virtual ~FWatchedSandboxRepository() override;
		
	//~ Begin ISandboxRepository Interface
	virtual void ForEachSandbox(TFunctionRef<EBreakBehavior(const FString& InRootPath, const FSandboxMetaInfo& MetaData)> InProcessSandboxes) override;
	virtual int32 NumSandboxes() const override { return CachedSandboxData.Num(); }
	virtual bool ReadMetaData(const FString& InRootPath, TFunctionRef<void(const FSandboxMetaInfo&)> InProcessMetadata) override;
	virtual FOnSandboxesChanged& OnSandboxesChanged() override { return OnSandboxesChangedDelegate; }
	virtual FOnSandboxMetaDataChanged& OnSandboxMetaDataChanged() override { return OnSandboxMetaDataChangedDelegate; }
	//~ End ISandboxRepository Interface

private:
	
	/** The base directory in which to look for sandboxes. Only looks at the top-level, i.e. not recursively. */
	const FString BaseDirectory;
 	
	/** Used to be notified when a sandbox directory is created since the sandbox watcher may not always fire immediately. */
	ISandboxManager& SandboxManager;
	
	struct FSandboxData
	{
		/** Absolute path to the sandbox root directory, i.e. the directory containing the metadata file. */
		FString AbsPath;
		
		/** Cached info about the sandbox, such as of the sandbox metadata file (to avoid constant JSON parsing, file loading, etc.) */ 
		FSandboxMetaInfo CachedData;
	};
	/** Cached sandbox data. Updated whenever info changes on disk. */
	TArray<FSandboxData> CachedSandboxData;
	
	/** Watcher for BaseDirectory */
	const FScopedWatchedDirectory WatchedBaseDirectory;
	/** 
	 * Sub-directories were watching in WatchedBaseDirectory. 
	 * All directories in the base directory are always watched so we can detect when a new sandbox is created there. 
	 */
	TArray<FScopedWatchedDirectory> WatchedSubDirectories;
	/** The directory that are supposed to be unregistered at end of frame. */
	TArray<FScopedWatchedDirectory> PendingDirectoryWatchersToUnregister;
	
	/** Invoked when the known sandboxes changes (removed, discovered, etc.) */
	FOnSandboxesChanged OnSandboxesChangedDelegate;
	/** Invoked when a sandbox's metadata changes. */
	FOnSandboxMetaDataChanged OnSandboxMetaDataChangedDelegate;
	
	/** Fully rebuilds all directory data, analzying all directory, parsing all metadata files, etc. */
	void RescanBaseDirectory();
	/** Adds CachedSandboxData entry if it contains a sandbox. Also starts listening for file changes. */
	void AddSandboxIfExists(const FString& InDirectory);
	
	/** @return Index to CachedSandboxData entry that contains the InDirectory (relative or absolute). */
	int32 IndexOf(const FString& InDirectory) const;
	
	/** Invoked when a change is made in the base directory. */
	void OnBaseDirectoryChanged(const TArray<FFileChangeData>& InChanges);
	/** Checks all reported changes made to directories with the goal of detecting added or removed sandboxes. */
	void UpdateCacheFromChangedBaseDirectory(const TArray<FFileChangeData>& InChanges);
	/** Ensures all direct sub-directories in the base directory have a directory watcher set up. */
	void UpdateWatchedDirectories();
	
	/** Invoked when a change is made in any of the sandbox directories. Used to update metadata. */
	void OnSandboxDirectoryChanged(const TArray<FFileChangeData>& InChanges, FString InSandboxPath);
	/** Checks whether a change has been reported on the metadata file. */
	bool DetectMetaDataFileChange(int32 InSandboxIndex, const TArray<FFileChangeData>& InChanges, const FString& InSandboxPath);
	/** Checks whether a change has been reported for the manifest file. */
	bool DetectManifestFileChange(int32 InSandboxIndex, const TArray<FFileChangeData>& InChanges, const FString& InSandboxPath);
	
	/** Handles a sandbox being created. */
	void OnSandboxStartup(ISandboxInstance& SandboxInstance);
	void OnEndOfFrame();
};	
}

#endif
