// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISandboxInstance.h"

#include "Misc/DateTime.h"
#include "Types/EBreakBehavior.h"
#include "Types/Sandbox/RevertArgs.h"
#include "Types/Sandbox/RevertResult.h"
#include "Types/Sandbox/PersistArgs.h"
#include "Utils/PackageSandboxUtils.h"

namespace UE::FileSandboxCore
{
bool ISandboxInstance::PersistAll()
{
	const FGatheredFileChanges FileChanges = GatherChangedFiles(EFileChangeGatherFlags::None);
	const TArray<FString>& ChangedFiles = FileChanges.NonSandboxPaths;
	FPersistResult Result = PersistSandbox(FPersistArgs{ ChangedFiles });
	return Result.PersistStatus == EPersistStatus::Success;
}

FGatheredFileChanges ISandboxInstance::GatherChangedFiles(EFileChangeGatherFlags InFlags) const
{
	FGatheredFileChanges Result;
	const EFileEnumerationFlags EnumerationFlags = EnumHasAnyFlags(InFlags, EFileChangeGatherFlags::IncludeTimestamps) 
		? EFileEnumerationFlags::IncludeTimestamps : EFileEnumerationFlags::None;
	EnumerateFileChanges([this, InFlags, &Result](const FSandboxedFileChangeInfo& InChange)
	{
		Result.NonSandboxPaths.Add(InChange.Path);
		if (EnumHasAnyFlags(InFlags, EFileChangeGatherFlags::IncludeChangeTypes))
		{
			Result.FileActions.Add(InChange.Action);
		}
		if (EnumHasAnyFlags(InFlags, EFileChangeGatherFlags::IncludeTimestamps))
		{
			Result.Timestamps.Add(InChange.Timestamp);
		}
		return EBreakBehavior::Continue;
	}, EnumerationFlags);
	return Result;
}

bool ISandboxInstance::HasFileChanges() const
{
	bool bHasChanges = false;
	EnumerateFileChanges([&bHasChanges](const FSandboxedFileChangeInfo& InChange)
	{
		bHasChanges = true;
		return EBreakBehavior::Break;
	});
	return bHasChanges;
}
}
