// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiffUtils.h"

#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Data/ManifestData.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Sandbox/Platform/SanboxedFilePathUtils.h"
#include "Sandbox/Platform/SandboxMountPoint.h"
#include "Templates/Function.h"
#include "Types/EBreakBehavior.h"
#include "Types/SandboxFileChange.h"
#include "Utils/SandboxFileUtils.h"

namespace UE::FileSandboxCore
{
namespace DiffDetail
{
template<typename TRangeType>
static void AnalyzeFilesMarkedRemoved(
	IPlatformFile& InPlatformFile, const TRangeType& InFilesMarkedForRemove, 
	TFunctionRef<EBreakBehavior(const FNonSandboxPath&)> InHandleRemovedFile
	)
{
	for (const FNonSandboxPath& PathMarkedRemoved : InFilesMarkedForRemove)
	{
		if (InPlatformFile.FileExists(*PathMarkedRemoved)
			&& InHandleRemovedFile(PathMarkedRemoved) == EBreakBehavior::Break)
		{
			return;
		}
	}
}
}

void AnalyzeFilesMarkedRemoved(
	IPlatformFile& InPlatformFile,
	const TSet<FNonSandboxPath>& InFilesMarkedForRemove, 
	TFunctionRef<EBreakBehavior(const FNonSandboxPath&)> InHandleRemovedFile
	)
{
	DiffDetail::AnalyzeFilesMarkedRemoved(InPlatformFile, InFilesMarkedForRemove, InHandleRemovedFile);
}

void AnalyzeFilesMarkedRemoved(
	IPlatformFile& InPlatformFile, const TConstArrayView<FNonSandboxPath>& InFilesMarkedForRemove,
	TFunctionRef<EBreakBehavior(const FNonSandboxPath&)> InHandleRemovedFile
	)
{
	DiffDetail::AnalyzeFilesMarkedRemoved(InPlatformFile, InFilesMarkedForRemove, InHandleRemovedFile);
}

void ScanForEditedOrAddedFiles(
	IPlatformFile& InPlatformFile,
	const FSandboxedPlatformFilePath& InSandboxMountPoint,
	TFunctionRef<FHandleFilePathSignature> InHandleEditedFile, 
	TFunctionRef<FHandleFilePathSignature> InHandleAddedFile
	)
{
	InPlatformFile.IterateDirectoryRecursively(*InSandboxMountPoint.GetSandboxPath(), 
		[&InPlatformFile, &InSandboxMountPoint, &InHandleEditedFile, &InHandleAddedFile]
		(const TCHAR* InFilenameOrDirectory, const bool InIsDirectory) -> bool
		{
			if (InIsDirectory)
			{
				return true;
			}

			const FSandboxedPlatformFilePath Path = FSandboxedPlatformFilePath::CreateNonSandboxPath(
				FPaths::ConvertRelativePathToFull(InFilenameOrDirectory), InSandboxMountPoint
				);
			const FString& NonSandboxPath = Path.GetNonSandboxPath();
					
			const bool bWasEdited = InPlatformFile.FileExists(*NonSandboxPath);
			if (bWasEdited)
			{
				return InHandleEditedFile(Path) == EBreakBehavior::Continue;
			}
			else
			{
				return InHandleAddedFile(Path) == EBreakBehavior::Continue;
			}
		});
}

void ScanForEditedOrAddedFiles(
	IPlatformFile& InPlatformFile,
	TConstArrayView<FSandboxMountPoint> InSandboxMountPoints, 
	TFunctionRef<FHandleFilePathSignature> InHandleEditedFile,
	TFunctionRef<FHandleFilePathSignature> InHandleAddedFile
	)
{
	for (const FSandboxMountPoint& MountPoint : InSandboxMountPoints)
	{
		ScanForEditedOrAddedFiles(InPlatformFile, MountPoint.Path, InHandleEditedFile, InHandleAddedFile);
	}
}

void EnumerateFileChanges(
	IPlatformFile& InPlatformFile,
	const FString& InSandboxRootPath, 
	const FFileSandboxCore_ManifestData& InManifest,
	TFunctionRef<FProcessFileChangeSignature> InProcess,
	EFileEnumerationFlags InFlags
	)
{
	const auto ProcessChange = [&InSandboxRootPath, &InManifest, &InProcess, InFlags](const FString& InNonSandboxPath, ESandboxFileChange InAction)
	{
		FSandboxedFileChangeInfo Info{ InNonSandboxPath, InAction };
		if (EnumHasAnyFlags(InFlags, EFileEnumerationFlags::IncludeTimestamps))
		{
			Info.Timestamp = GetSandboxTimestamp(InNonSandboxPath, InSandboxRootPath, InManifest).Get(FDateTime::MinValue());
		}
		return InProcess(Info);
	};
	
	EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
	AnalyzeFilesMarkedRemoved(InPlatformFile, InManifest.DeletedFiles.GetFiles(), [&ProcessChange, &BreakBehavior](const FString& InFile)
	{
		BreakBehavior = ProcessChange(InFile, ESandboxFileChange::Removed);
		return BreakBehavior;
	});
	if (BreakBehavior == EBreakBehavior::Break)
	{
		return;
	}
	
	const auto ProcessMountPoint = [&InSandboxRootPath, &ProcessChange, &InPlatformFile, &BreakBehavior]
		(const FString& InAssetPath, const FString& InFilesystemPath)
	{
		const FSandboxedPlatformFilePath MountPoint = FSandboxedPlatformFilePath::CreateMountPoint(
			GetSandboxMountPointRoot(InSandboxRootPath), InAssetPath, InFilesystemPath
		);
			
		const auto HandleEdited = [&ProcessChange, &BreakBehavior](const FSandboxedPlatformFilePath& InPath)
		{
			BreakBehavior = ProcessChange(InPath.GetNonSandboxPath(), ESandboxFileChange::Edited);
			return BreakBehavior;
		};
		const auto HandleAdded = [&ProcessChange, &BreakBehavior](const FSandboxedPlatformFilePath& InPath)
		{
			BreakBehavior = ProcessChange(InPath.GetNonSandboxPath(), ESandboxFileChange::Added);
			return BreakBehavior;
		};
			
		ScanForEditedOrAddedFiles(InPlatformFile, MountPoint, HandleEdited, HandleAdded);
		return BreakBehavior;
	};
	EnumerateMountPoints(ProcessMountPoint);
}
}
