// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISandboxManager.h"
#include "Containers/ContainersFwd.h"
#include "Data/ManifestData.h"
#include "Sandbox/Platform/SandboxedPlatformFilePath.h"
#include "Templates/FunctionFwd.h"
#include "Types/EBreakBehavior.h"

struct FFileSandboxCore_ManifestData;

namespace UE::FileSandboxCore
{
enum class EBreakBehavior : uint8;
struct FSandboxMountPoint;

using FHandleFilePathSignature = EBreakBehavior(const FSandboxedPlatformFilePath& InPath);

/** 
 * Analyzes the file action the sandbox system would perform on files previously marked for remove.
 * This compares whether the non-sandbox path still exists, i.e. whether the file was already removed (externally).
 * 
 * This is useful if you're the state from the manifest file and need to compare against the current disk state.
 * 
 * @param InPlatformFile The lower level platform file, i.e. the one that FSandboxPlatformFile wraps.
 * @param InFilesMarkedForRemove The files to analyze.
 * @param InHandleRemovedFile Invoked for each file that still should be marked as removed.
 */
void AnalyzeFilesMarkedRemoved(
	IPlatformFile& InPlatformFile,
	const TSet<FNonSandboxPath>& InFilesMarkedForRemove, 
	TFunctionRef<EBreakBehavior(const FNonSandboxPath&)> InHandleRemovedFile
	);
void AnalyzeFilesMarkedRemoved(
	IPlatformFile& InPlatformFile,
	const TConstArrayView<FNonSandboxPath>& InFilesMarkedForRemove, 
	TFunctionRef<EBreakBehavior(const FNonSandboxPath&)> InHandleRemovedFile
	);

/** 
 * Recursively iterates the files in the mount point and determines whether the present files would be
 * - editing the non-sandbox counterpart, or
 * - adding a new file to the non-sandbox.
 * This is useful if you're the state from the manifest file and need to compare against the current disk state.
 * 
 * @param InPlatformFile The lower level platform file, i.e. the one that FSandboxPlatformFile wraps.
 * @param InSandboxMountPoint The sandbox mountpoint
 * @param InHandleEditedFile Invoked for each file that should be marked as edited.
 * @param InHandleAddedFile Invoked for each file that should be marked as added.
 */
void ScanForEditedOrAddedFiles(
	IPlatformFile& InPlatformFile,
	const FSandboxedPlatformFilePath& InSandboxMountPoint,
	TFunctionRef<FHandleFilePathSignature> InHandleEditedFile,
	TFunctionRef<FHandleFilePathSignature> InHandleAddedFile
	);

/** Utility overload for FSandboxPlatformFile to pass in all of its mount points. */
void ScanForEditedOrAddedFiles(
	IPlatformFile& InPlatformFile,
	TConstArrayView<FSandboxMountPoint> InSandboxMountPoints,
	TFunctionRef<FHandleFilePathSignature> InHandleEditedFile,
	TFunctionRef<FHandleFilePathSignature> InHandleAddedFile
	);

/** 
 * Shared logic for determining the effective file changes.
 * @see ISandboxInstance::EnumerateFileChanges.
 * @see ISandboxManager::EnumerateFileChanges.
 */
void EnumerateFileChanges(
	IPlatformFile& InPlatformFile, const FString& InSandboxRootPath, const FFileSandboxCore_ManifestData& InManifest, 
	TFunctionRef<EBreakBehavior(const FSandboxedFileChangeInfo& InFileInfo)> InProcess,
	EFileEnumerationFlags InFlags = EFileEnumerationFlags::All
	);
}
