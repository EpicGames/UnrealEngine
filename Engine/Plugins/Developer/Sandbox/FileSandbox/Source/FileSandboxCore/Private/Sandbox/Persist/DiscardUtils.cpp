// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiscardUtils.h"

#include "DiffUtils.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Sandbox/Platform/SandboxMountPoint.h"
#include "UObject/NameTypes.h"
#include "Utils/FileChange/FileChange.h"
#include "Utils/AssetUtils.h"

namespace UE::FileSandboxCore
{
namespace DiscardDetail
{
static bool ShouldDiscardFile(TConstArrayView<FString> InFilesToDiscard, const FString& InFile)
{
	return InFilesToDiscard.IsEmpty() || InFilesToDiscard.ContainsByPredicate([&InFile](const FString& InDiscardableFile)
	{
		return FPaths::IsSamePath(InDiscardableFile, InFile);
	});
}
static bool ShouldDiscardFile(TConstArrayView<FString> InFilesToDiscard, const FSandboxedPlatformFilePath& InFile)
{
	return ShouldDiscardFile(InFilesToDiscard, InFile.GetNonSandboxPath());
}
}

void DiscardAddedAndModifiedPaths(
	TConstArrayView<FString> InFilesToDiscard, 
	TConstArrayView<FSandboxMountPoint> InMountPoints, IPlatformFile& InLowerLevel, 
	TArray<FName>& OutPackagesPendingHotReload, TArray<FName>& OutPackagesPendingPurge,
	TFunctionRef<void(const FSandboxedPlatformFilePath& Path, EFileChangeAction Action)> InProcessFileChangeCallback
	)
{
	const auto HandleEditedFile = [&](const FSandboxedPlatformFilePath& InPath)
	{
		if (!DiscardDetail::ShouldDiscardFile(InFilesToDiscard, InPath))
		{
			return EBreakBehavior::Continue;
		}
			
		// If this file maps to a package then we need to flush its linker so that we can remove the file from the sandbox
		FName PackageName;
		FlushPackageFile(InPath.GetNonSandboxPath(), &PackageName, false);
			
		if (!PackageName.IsNone())
		{
			OutPackagesPendingPurge.Remove(PackageName);
			OutPackagesPendingHotReload.Add(PackageName);
		}
				
		InProcessFileChangeCallback(InPath, EFileChangeAction::Modified);
		return EBreakBehavior::Continue;
	};
	
	const auto HandleAddedFile = [&](const FSandboxedPlatformFilePath& InPath)
	{
		if (!DiscardDetail::ShouldDiscardFile(InFilesToDiscard, InPath))
		{
			return EBreakBehavior::Continue;
		}
			
		// If this file maps to a package then we need to flush its linker so that we can remove the file from the sandbox
		FName PackageName;
		FlushPackageFile(InPath.GetNonSandboxPath(), &PackageName, false);
			
		if (!PackageName.IsNone())
		{
			OutPackagesPendingPurge.Add(PackageName);
			OutPackagesPendingHotReload.Remove(PackageName);
		}
				
		InProcessFileChangeCallback(InPath, EFileChangeAction::Removed);
		return EBreakBehavior::Continue;
	};
	
	ScanForEditedOrAddedFiles(InLowerLevel, InMountPoints, HandleEditedFile, HandleAddedFile);

	//FString SandboxRootPath = GetSandboxRootPath();
	//FString Stashed = SandboxRootPath.Replace(TEXT("Sandbox"), TEXT("Stashed"));
	//LowerLevel->MoveFile(SandboxRootPath, Stashed);
	// Delete everything under the mount point
	//LowerLevel->IterateDirectory(*SandboxMountPoint.Path.GetSandboxPath(), [this](const TCHAR* InFilenameOrDirectory, const bool InIsDirectory) -> bool
	//{
	//	if (InIsDirectory)
	//	{
	//		LowerLevel->DeleteDirectoryRecursively(InFilenameOrDirectory);
	//	}
	//	else
	//	{
	//		LowerLevel->DeleteFile(InFilenameOrDirectory);
	//	}
	//	return true; // Continue iteration
	//});
}
}
