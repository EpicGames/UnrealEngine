// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Sandbox/Platform/SandboxedPlatformFilePath.h"
#include "Templates/Function.h"

class IPlatformFile;

namespace UE::FileSandboxCore
{
enum class EFileChangeAction : uint8;
struct FSandboxMountPoint;

/**
 * Discards files that have been added or edited in the sandbox.
 * @param InFilesToDiscard Relative or absolute non-sandbox paths. If left empty, revert all files.  
 */
void DiscardAddedAndModifiedPaths(
	TConstArrayView<FString> InFilesToDiscard, 
	TConstArrayView<FSandboxMountPoint> InMountPoints, IPlatformFile& InLowerLevel, 
	TArray<FName>& OutPackagesPendingHotReload, TArray<FName>& OutPackagesPendingPurge,
	TFunctionRef<void(const FSandboxedPlatformFilePath& Path, EFileChangeAction Action)> InProcessFileChangeCallback = [](const FSandboxedPlatformFilePath&, EFileChangeAction){}
	);
}
