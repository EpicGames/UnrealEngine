// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "HAL/PlatformMisc.h"
#include "Interface/IPersistFeedback.h"
#include "LogFileSandbox.h"
#include "Logging/LogMacros.h"

namespace UE::FileSandboxCore
{
class FLogPersistFeedback : public IPersistFeedback
{
public:
	
	//~ Begin IPersistFeedback Interface
	virtual void StartFile(const TCHAR* InFilename) override
	{
		UE_LOGF(LogFileSandbox, Verbose, "Starting persist for file '%ls'", InFilename);
	}
	virtual void HandleSuccess(const TCHAR* InFilename) override
	{
		UE_LOGF(LogFileSandbox, Verbose, "Persisted file '%ls'", InFilename);
	}
	virtual void HandleError_CheckoutNotAllowed(const TCHAR* InFilename) override
	{
		UE_LOGF(LogFileSandbox, Warning, "Checkout of file '%ls' from revision control was not allowed when persisting sandbox state!", InFilename);
	}
	virtual void HandleError_Checkout(const TCHAR* InFilename) override
	{
		UE_LOGF(LogFileSandbox, Warning, "Failed to checkout file '%ls' from revision control when persisting sandbox state!", InFilename);
	}
	virtual void HandleError_Revert(const TCHAR* InFilename) override
	{
		UE_LOGF(LogFileSandbox, Warning, "Failed to revert file '%ls' from revision control when persisting sandbox state!", InFilename);
	}
	virtual void HandleError_MarkForAdd(const TCHAR* InFilename) override
	{
		UE_LOGF(LogFileSandbox, Warning, "Failed to add file '%ls' to revision control when persisting sandbox state!", InFilename);
	}
	virtual void HandleError_DeleteSCC(const TCHAR* InFilename) override
	{
		UE_LOGF(LogFileSandbox, Warning, "Failed to delete file '%ls' from revision control when persisting sandbox state!", InFilename);
	}
	virtual void HandleError_MakeWritable(const TCHAR* InFilename) override
	{
		UE_LOGF(LogFileSandbox, Warning, "Failed to make '%ls' writeable using file system when persisting sandbox state! %ls", InFilename, *GetErrorMessage());
	}
	virtual void HandleError_MoveFile(const TCHAR* InToFilename, const TCHAR* InFromFilename) override
	{
		const uint32 LastErrorCode = FPlatformMisc::GetLastError();
		UE_LOGF(LogFileSandbox, Warning, "Failed to move file '%ls' to '%ls' using file system when persisting sandbox state! %ls", InFromFilename, InToFilename, *GetErrorMessage());
	}
	virtual void HandleError_DeleteFile(const TCHAR* InFilename) override
	{
		const uint32 LastErrorCode = FPlatformMisc::GetLastError();
		UE_LOGF(LogFileSandbox, Warning, "Failed to delete file '%ls' using file system when persisting sandbox state! %ls", InFilename, *GetErrorMessage());
	}
	//~ End IPersistFeedback Interface
	
private:
	
	static FString GetErrorMessage()
	{
		const uint32 LastErrorCode = FPlatformMisc::GetLastError();
		TCHAR ErrorBuffer[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, LastErrorCode);
		return FString::Printf(TEXT("GetLastError: %d - %s"), LastErrorCode, ErrorBuffer);
	}
};
}
