// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersistUtils.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Interface/IPersistFeedback.h"
#include "Misc/Paths.h"
#include "SourceControlOperations.h"
#include "Utils/AssetUtils.h"

namespace UE::FileSandboxCore
{
namespace PersistDetail
{
static bool TryCheckout(
	ISourceControlProvider& InSourceControlProvider, IPersistFeedback& InErrorHandler,
	const TCHAR* InToFilename, const FSourceControlStateRef& ToFileSCCState
	)
{
	// Handles e.g. exclusive checkout
	const bool bIsCheckoutDisallowed = InSourceControlProvider.UsesCheckout() && !ToFileSCCState->IsCheckedOut() && !ToFileSCCState->CanCheckout();
	if (bIsCheckoutDisallowed)
	{
		InErrorHandler.HandleError_CheckoutNotAllowed(InToFilename);
		return false;
	}
		
	if (ToFileSCCState->CanCheckout() && InSourceControlProvider.UsesCheckout())
	{
		TArray<FString> FilesToBeCheckedOut;
		FilesToBeCheckedOut.Add(InToFilename);

		if (InSourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToBeCheckedOut) != ECommandResult::Succeeded)
		{
			InErrorHandler.HandleError_Checkout(InToFilename);
			return false;
		}
	}
	
	return true;
}
}

bool AddFileWithSCC(
	IPlatformFile& InPlatformFile, ISourceControlProvider& InSourceControlProvider, IPersistFeedback& InErrorHandler,
	const TCHAR* InToFilename, const TCHAR* InFromFilename,
	EPersistFlags InFlags
	)
{
	if (!MoveSandboxFileToNonSandbox(InPlatformFile, InErrorHandler, InToFilename, InFromFilename, InFlags))
	{
		return false;
	}
	
	// If the file is new, add it to source control now
	const FSourceControlStatePtr ToFileSCCState = InSourceControlProvider.IsEnabled() 
		? InSourceControlProvider.GetState(InToFilename, EStateCacheUsage::ForceUpdate) : nullptr;
	if (ToFileSCCState && !ToFileSCCState->IsSourceControlled())
	{
		const TArray<FString> FilesToBeAdded { InToFilename };
		if (InSourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilesToBeAdded) != ECommandResult::Succeeded)
		{
			InErrorHandler.HandleError_MarkForAdd(InToFilename);
			return false;
		}
	}
	// If the file was deleted before we entered the sandbox, but in the sandbox we added it, it means the file should be edited.
	else if (ToFileSCCState && ToFileSCCState->IsDeleted())
	{
		if (InSourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), InToFilename) != ECommandResult::Succeeded)
		{
			InErrorHandler.HandleError_Revert(InToFilename);
			return false;
		}
		
		return PersistDetail::TryCheckout(InSourceControlProvider, InErrorHandler, InToFilename, ToFileSCCState.ToSharedRef());
	}
	
	return true;
}

bool EditFileWithSCC(
	IPlatformFile& InPlatformFile, ISourceControlProvider& InSourceControlProvider, IPersistFeedback& InErrorHandler,
	const TCHAR* InToFilename, const TCHAR* InFromFilename, 
	bool bMarkWriteable, 
	EPersistFlags InFlags
	)
{
	const FSourceControlStatePtr ToFileSCCState = InSourceControlProvider.IsEnabled() 
		? InSourceControlProvider.GetState(InToFilename, EStateCacheUsage::ForceUpdate) : nullptr;
	
	if (ToFileSCCState && ToFileSCCState->IsSourceControlled()) 
	{
		// If the file was marked for add before we edited it in sandbox, it should remain marked for add.
		if (!ToFileSCCState->IsAdded() 
			&& !PersistDetail::TryCheckout(InSourceControlProvider, InErrorHandler, InToFilename, ToFileSCCState.ToSharedRef()))
		{
			return false;
		}
	}
	else if (bMarkWriteable)
	{
		if (InPlatformFile.FileExists(InToFilename) && !InPlatformFile.SetReadOnly(InToFilename, false))
		{
			InErrorHandler.HandleError_MakeWritable(InToFilename);
			return false;
		}
	}
	
	return MoveSandboxFileToNonSandbox(InPlatformFile, InErrorHandler, InToFilename, InFromFilename, InFlags);
}

bool DeleteFileWithSCC(
	IPlatformFile& InPlatformFile, ISourceControlProvider& InSourceControlProvider, IPersistFeedback& InErrorHandler, const TCHAR* InFilename
	)
{
	// If this file maps to a package then we need to flush its linker so that we can remove the file from disk
	FlushPackageFile(InFilename);
	
	// Get the source control state of the file
	const FSourceControlStatePtr FileSCCState = InSourceControlProvider.IsEnabled() 
		? InSourceControlProvider.GetState(InFilename, EStateCacheUsage::ForceUpdate) : nullptr;
	// Try and let source control remove the file first
	if (FileSCCState && FileSCCState->IsSourceControlled())
	{
		const bool bAdded = FileSCCState->IsAdded();

		const bool bCanRevert = bAdded || FileSCCState->IsCheckedOut(); 
		if (bCanRevert && InSourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), InFilename) != ECommandResult::Succeeded)
		{
			InErrorHandler.HandleError_Revert(InFilename);
			return false;
		}
		
		// If the file was marked for add, then revert is guaranteed to remove the file; it is invalid to call delete on that file again.
		if (!bAdded && InSourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), InFilename) != ECommandResult::Succeeded)
		{
			InErrorHandler.HandleError_DeleteSCC(InFilename);
			return false;
		}
	}

	// Delete file if it still exists
	if (InPlatformFile.FileExists(InFilename) && !InPlatformFile.DeleteFile(InFilename))
	{
		InErrorHandler.HandleError_DeleteFile(InFilename);
		return false;
	}
	return true;
}

namespace PersistDetail
{
static bool CopyFile(IPlatformFile& InPlatformFile, IPersistFeedback& InErrorHandler, const TCHAR* InToFilename, const TCHAR* InFromFilename)
{
	if (!InPlatformFile.CopyFile(InToFilename, InFromFilename, EPlatformFileRead::None, EPlatformFileWrite::AttemptDeleteAndCreate))
	{
		InErrorHandler.HandleError_MoveFile(InToFilename, InFromFilename);
		return false;
	}
	return true;
}

static bool MoveFile(IPlatformFile& InPlatformFile, IPersistFeedback& InErrorHandler, const TCHAR* InToFilename, const TCHAR* InFromFilename)
{
	bool bSuccess = true;
		
	// If this file maps to a package then we need to flush its linker so that we can remove the file from disk
	FlushPackageFile(InToFilename);
	
	// MoveFile fails if there already is a target file
	if (InPlatformFile.FileExists(InToFilename))
	{
		// DeleteFile fails if the file is read only
		bSuccess &= !InPlatformFile.IsReadOnly(InToFilename) || InPlatformFile.SetReadOnly(InToFilename, false);
		bSuccess &= InPlatformFile.DeleteFile(InToFilename);
	}
	
	bSuccess &= InPlatformFile.MoveFile(InToFilename, InFromFilename);
	
	if (!bSuccess)
	{
		InErrorHandler.HandleError_MoveFile(InToFilename, InFromFilename);
	}
		
	return bSuccess;
}
}

bool MoveSandboxFileToNonSandbox(
	IPlatformFile& InPlatformFile, IPersistFeedback& InErrorHandler,
	const TCHAR* InToFilename, const TCHAR* InFromFilename,
	EPersistFlags InFlags
	)
{
	const FString ToFileDir = FPaths::GetPath(InToFilename);
	if (!InPlatformFile.CreateDirectoryTree(*ToFileDir))
	{
		InErrorHandler.HandleError_MoveFile(InToFilename, InFromFilename);
		return false;
	}
	
	return EnumHasAnyFlags(InFlags, EPersistFlags::RetainChangedFiles)
		? PersistDetail::CopyFile(InPlatformFile, InErrorHandler, InToFilename, InFromFilename)
		: PersistDetail::MoveFile(InPlatformFile, InErrorHandler, InToFilename, InFromFilename);
}
}
