// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

class IPlatformFile;
class ISourceControlProvider;

namespace UE::FileSandboxCore
{
class FSandboxedPlatformFilePath;
class IPersistFeedback;
enum class ESandboxFileChange : uint8;
struct FPersistArgs;

enum class EPersistFlags : uint8
{
	None,
	
	/** 
	 * If set, added & edited files are retained in the sandbox directory on persist, i.e. sandboxed files are copied to the non-sandbox files.
	 * If unset, added & edited files are removed from the sandbox directory on persist, i.e. sandboxed files are moved to the non-sandbox files.
	 */
	RetainChangedFiles = 1 << 0
};
ENUM_CLASS_FLAGS(EPersistFlags);

/** Persists a file that is missing from the non-sandbox but exists in the sandbox. Registers the changes with Source Control if enabled. */
bool AddFileWithSCC(
	IPlatformFile& InPlatformFile, ISourceControlProvider& InSourceControlProvider, IPersistFeedback& InErrorHandler,
	const TCHAR* InToFilename, const TCHAR* InFromFilename, 
	EPersistFlags InFlags = EPersistFlags::None
	);
/** Persists a file that exists in both the non-sandbox and sandbox. Registers the changes with Source Control if enabled. */
bool EditFileWithSCC(
	IPlatformFile& InPlatformFile, ISourceControlProvider& InSourceControlProvider, IPersistFeedback& InErrorHandler,
	const TCHAR* InToFilename, const TCHAR* InFromFilename, 
	bool bMarkWriteable, 
	EPersistFlags InFlags = EPersistFlags::None
	);
/** Persists a file that exists in non-sandbox but was removed in the sandbox. Registers the changes with Source Control if enabled. */
bool DeleteFileWithSCC(
	IPlatformFile& InPlatformFile, ISourceControlProvider& InSourceControlProvider, IPersistFeedback& InErrorHandler,
	const TCHAR* InFilename
	);

/** 
 * Moves InFromFilename to InToFileName. 
 * 
 * After successful operation, InToFileName has the file content that InFromFilename previously had. 
 * Depending on the flags InFromFilename stays in the sandbox, and continues to be tracked as change, or is removed.
 */
bool MoveSandboxFileToNonSandbox(
	IPlatformFile& PlatformFile, IPersistFeedback& InErrorHandler, 
	const TCHAR* InToFilename, const TCHAR* InFromFilename, 
	EPersistFlags InFlags = EPersistFlags::None
	);
}
