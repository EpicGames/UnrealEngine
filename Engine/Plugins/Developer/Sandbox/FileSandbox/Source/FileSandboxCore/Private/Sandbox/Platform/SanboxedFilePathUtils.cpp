// Copyright Epic Games, Inc. All Rights Reserved.

#include "SanboxedFilePathUtils.h"

#include "SandboxedPlatformFilePath.h"
#include "Misc/Optional.h"

namespace UE::FileSandboxCore
{
bool IsFileInMountPoint(const FString& InNonSandboxFilePath, const FSandboxedPlatformFilePath& InMountPointRoot)
{
	// Mount point are stored with a trailing slash to prevent matching mount point with similar names -> (/Bla/Content, /Bla/ContentSupreme)
	// So we test without the slash to make sure with can match mount point directly -> (/Bla/Content match /Bla/Content/)
	const FString& PathStr = InMountPointRoot.GetNonSandboxPath();
	int32 PathStrNoSlashLength = PathStr.Len() - 1;
	return FCString::Strnicmp(*InNonSandboxFilePath, *PathStr, PathStrNoSlashLength) == 0 
		&& (InNonSandboxFilePath.Len() == PathStrNoSlashLength 
			|| InNonSandboxFilePath[PathStrNoSlashLength] == TEXT('/')
			);
}

TOptional<FSandboxedPlatformFilePath> MakeSandboxedFilePathIfInMountPoint(
	const FString& InNonSandboxFilePath, const FSandboxedPlatformFilePath& InMountPointRoot
	)
{
	return IsFileInMountPoint(InNonSandboxFilePath, InMountPointRoot)
		? FSandboxedPlatformFilePath::CreateSandboxPath(FString(InNonSandboxFilePath), InMountPointRoot)
		: TOptional<FSandboxedPlatformFilePath>();
}
}
