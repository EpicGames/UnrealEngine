// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template<typename OptionalType> struct TOptional;

class FString;

namespace UE::FileSandboxCore
{
class FSandboxedPlatformFilePath;

/** @return Whether InNonSandboxFilePath is a file path to InMountPointRoot */
bool IsFileInMountPoint(const FString& InNonSandboxFilePath, const FSandboxedPlatformFilePath& InMountPointRoot);

/** @return Sandboxed file path if InNonSandboxFilePath is under InMountPointRoot. */
TOptional<FSandboxedPlatformFilePath> MakeSandboxedFilePathIfInMountPoint(
	const FString& InNonSandboxFilePath, const FSandboxedPlatformFilePath& InMountPointRoot
	);
}

