// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SandboxedPlatformFilePath.h"
#include "Delegates/IDelegateInstance.h"

namespace UE::FileSandboxCore
{
struct FSandboxMountPoint
{
	/** Sandbox path */
	FSandboxedPlatformFilePath Path;

	/** Sandbox directory watcher delegate handle (if any) */
	FDelegateHandle OnDirectoryChangedHandle;
};
}
