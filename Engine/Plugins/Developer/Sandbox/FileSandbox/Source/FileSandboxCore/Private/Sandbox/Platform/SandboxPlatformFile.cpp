// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxPlatformFile.h"

#include "Containers/UnrealString.h"
#include "Engine/GameEngine.h"
#include "Engine/World.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "ISourceControlProvider.h"
#include "Interface/IPersistFeedback.h"
#include "JsonObjectConverter.h"
#include "LogFileSandbox.h"
#include "Logging/LogVerbosity.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"
#include "SanboxedFilePathUtils.h"
#include "Sandbox/Persist/DiffUtils.h"
#include "Sandbox/Persist/DiscardUtils.h"
#include "Sandbox/Persist/PersistUtils.h"
#include "SandboxMountPoint.h"
#include "Types/EBreakBehavior.h"
#include "Types/Sandbox/PersistArgs.h"
#include "Types/SandboxFileChange.h"
#include "UObject/Package.h"
#include "Utils/AssetUtils.h"
#include "Utils/FileChange/FileChange.h"
#include "Utils/SandboxFileUtils.h"

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
#include "Utils/Watcher/DirectoryWatcherUtils.h"
#include "IDirectoryWatcher.h"
#endif

#if WITH_EDITOR
#include "DirectoryWatcherModule.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "FileHelpers.h"
#include "Selection.h"
#endif

class IDirectoryWatcher;

namespace UE::FileSandboxCore
{
/**
 * Use this to force a sync of the Asset Registry immediately to process the file changes. 
 * This ensures that the Asset Registry is brought up to date especially after file/package deletes, since otherwise it may not get updated before 
 * packages are hot reloaded and stale entries could still be present in the registry.
 */
class FScopedRegisterExternalFileChanges
{
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
	TArray<FFileChangeData> FileChanges;
#endif
public:
	
	~FScopedRegisterExternalFileChanges()
	{
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
		FDirectoryWatcherModule* DirectoryWatcherModule = GetDirectoryWatcherModuleIfLoaded();
		if (DirectoryWatcherModule && FileChanges.Num() > 0)
		{
			DirectoryWatcherModule->RegisterExternalChanges(FileChanges);
			SynchronizeAssetRegistry();
		}
#endif
	}
	
	void RegisterFileChange(const FFileChange& Change)
	{
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
		FileChanges.Emplace(Change.ToDirectoryWatcherChange());
#endif
	}
};

/** Only registers added or edited file action. Only */
static void RegisterAddedOrEditedOnPersist(
	FScopedRegisterExternalFileChanges& InChanges, ESandboxFileChange InActionPerformed, const FSandboxedPlatformFilePath& InFilePath
	)
{
	if (InActionPerformed == ESandboxFileChange::Added)
	{
		InChanges.RegisterFileChange(FFileChange(InFilePath.GetNonSandboxPath(), EFileChangeAction::Added));
	}
	else if (InActionPerformed == ESandboxFileChange::Edited)
	{
		InChanges.RegisterFileChange(FFileChange(InFilePath.GetNonSandboxPath(), EFileChangeAction::Modified));
	}
}

FSandboxPlatformFile::FSandboxPlatformFile(const FString& InSandboxRootPath)
	: RootPath(InSandboxRootPath)
	, LowerLevel(nullptr)
	, bSandboxEnabled(false)
{
}

FSandboxPlatformFile::~FSandboxPlatformFile()
{
	IPlatformFile& CurrentPlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (LowerLevel && this == &CurrentPlatformFile)
	{
		FPlatformFileManager::Get().SetPlatformFile(*LowerLevel);
	}

	FPackageName::OnContentPathMounted().RemoveAll(this);
	FPackageName::OnContentPathDismounted().RemoveAll(this);

#if WITH_EDITOR
	if (IDirectoryWatcher* DirectoryWatcher = GetDirectoryWatcherIfLoaded())
	{
		for (FSandboxMountPoint& SandboxMountPoint : SandboxMountPoints)
		{
			if (SandboxMountPoint.OnDirectoryChangedHandle.IsValid())
			{
				DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(SandboxMountPoint.Path.GetSandboxPath(), SandboxMountPoint.OnDirectoryChangedHandle);
				SandboxMountPoint.OnDirectoryChangedHandle.Reset();
			}
		}
	}
#endif
}

FString FSandboxPlatformFile::GetSandboxRootPath() const
{
	return GetSandboxMountPointRoot(RootPath);
}

void FSandboxPlatformFile::SetSandboxEnabled(bool bInEnabled)
{
	bSandboxEnabled = bInEnabled;
}

bool FSandboxPlatformFile::IsSandboxEnabled() const
{
	return bSandboxEnabled;
}

bool FSandboxPlatformFile::Initialize(IPlatformFile* Inner, const TCHAR* CmdLine)
{
	SetLowerLevel(Inner);
	
	LowerLevel->CreateDirectoryTree(*GetSandboxRootPath());

	// Set-up the initial set of content mount paths
	EnumerateMountPoints([this](const FString& InAssetPath, const FString& InFilesystemPath)
	{
		RegisterContentMountPath(InAssetPath, InFilesystemPath);
		return EBreakBehavior::Continue;
	});

	// Watch for new content mount paths
	FPackageName::OnContentPathMounted().AddRaw(this, &FSandboxPlatformFile::OnContentPathMounted);
	FPackageName::OnContentPathDismounted().AddRaw(this, &FSandboxPlatformFile::OnContentPathDismounted);

	bSandboxEnabled = true;
	FPlatformFileManager::Get().SetPlatformFile(*this);

#if WITH_EDITOR
	SynchronizeAssetRegistry();
#endif

	return true;
}

void FSandboxPlatformFile::Tick()
{
}

IPlatformFile* FSandboxPlatformFile::GetLowerLevel()
{
	return LowerLevel;
}

void FSandboxPlatformFile::SetLowerLevel(IPlatformFile* NewLowerLevel)
{
	check(NewLowerLevel && NewLowerLevel != this);
	LowerLevel = NewLowerLevel;
}

const TCHAR* FSandboxPlatformFile::GetName() const
{
	return TEXT("FSandboxPlatformFile");
}

bool FSandboxPlatformFile::FileExists(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return false;
		}

		if (LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return true;
		}
	}

	return LowerLevel->FileExists(*ResolvedPath.GetNonSandboxPath());
}

int64 FSandboxPlatformFile::FileSize(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return -1;
		}

		if (LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->FileSize(*ResolvedPath.GetSandboxPath());
		}
	}

	return LowerLevel->FileSize(*ResolvedPath.GetNonSandboxPath());
}

bool FSandboxPlatformFile::DeleteFile(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		// Cannot delete again.
		if (IsPathDeleted(ResolvedPath))
		{
			return false;
		}

		// Is file present outside of sandbox? E.g. existed before we entered sandbox. If so, mark as deleted.
		const bool bIsDeletingNonSandboxFile = LowerLevel->FileExists(*ResolvedPath.GetNonSandboxPath());
		if (bIsDeletingNonSandboxFile)
		{
			MarkPathDeleted(ResolvedPath);
		}
		
		// File only present in sandbox? E.g. file created during sandbox, but then deleted again.
		const bool bIsDeletingFileCreatedInSandbox = LowerLevel->DeleteFile(*ResolvedPath.GetSandboxPath()); 
		
		const bool bPerformedDeleteOp = bIsDeletingNonSandboxFile || bIsDeletingFileCreatedInSandbox;
		if (bPerformedDeleteOp)
		{
			NotifyFileDeleted(ResolvedPath);

			const bool bWasEdit = bIsDeletingNonSandboxFile && bIsDeletingFileCreatedInSandbox;
			const bool bWasAdd = !bIsDeletingNonSandboxFile && bIsDeletingFileCreatedInSandbox;

			const ESandboxFileChange OldState = bWasEdit ? ESandboxFileChange::Edited : (bWasAdd ? ESandboxFileChange::Added : ESandboxFileChange::None);
			BroadcastFileChange(ResolvedPath, OldState, ESandboxFileChange::Removed);
		}
		
		return bPerformedDeleteOp; 
	}

	return LowerLevel->DeleteFile(*ResolvedPath.GetNonSandboxPath());
}

bool FSandboxPlatformFile::IsReadOnly(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return false;
		}

		if (LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->IsReadOnly(*ResolvedPath.GetSandboxPath());
		}

		return false; // Can always overwrite missing sandbox files
	}

	return LowerLevel->IsReadOnly(*ResolvedPath.GetNonSandboxPath());
}

bool FSandboxPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return false;
		}

		if (LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			if (bNewReadOnlyValue)
			{
				return true; // Do not allow sandbox files to be made read-only, but don't report failure
			}

			return LowerLevel->SetReadOnly(*ResolvedPath.GetSandboxPath(), bNewReadOnlyValue);
		}

		return false; // Do not attempt to modify the non-sandbox file
	}

	return LowerLevel->SetReadOnly(*ResolvedPath.GetNonSandboxPath(), bNewReadOnlyValue);
}

bool FSandboxPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	const FSandboxedPlatformFilePath ResolvedToPath = ToSandboxPath(To);
	const FSandboxedPlatformFilePath ResolvedFromPath = ToSandboxPath(From);

	if (ResolvedToPath.HasSandboxPath())
	{
		// Migrate any existing target file from outside the sandbox so the lower-level will fail to overwrite the existing file
		MigrateFileToSandbox(ResolvedToPath);

		if (ResolvedFromPath.HasSandboxPath())
		{
			// Sandbox -> Sandbox
			if (IsPathDeleted(ResolvedFromPath))
			{
				// Cannot move a deleted file
				return false;
			}

			bool bSuccess = false;
			if (LowerLevel->FileExists(*ResolvedFromPath.GetSandboxPath()))
			{
				// Moving an internal sandbox file - can move
				bSuccess = LowerLevel->MoveFile(*ResolvedToPath.GetSandboxPath(), *ResolvedFromPath.GetSandboxPath());
			}
			else
			{
				// Moving an external sandbox file - must copy
				bSuccess = LowerLevel->CopyFile(*ResolvedToPath.GetSandboxPath(), *ResolvedFromPath.GetNonSandboxPath());
			}

			if (bSuccess)
			{
				ClearPathDeleted(ResolvedToPath);
				MarkPathDeleted(ResolvedFromPath);
				NotifyFileDeleted(ResolvedFromPath);
			}

			return bSuccess;
		}
		else
		{
			// Non-sandbox -> Sandbox
			if (LowerLevel->MoveFile(*ResolvedToPath.GetSandboxPath(), *ResolvedFromPath.GetNonSandboxPath()))
			{
				const bool bIsEdit = LowerLevel->FileExists(*ResolvedToPath.GetNonSandboxPath());
				if (IsPathDeleted(ResolvedToPath))
				{
					constexpr bool bBroadcast = false;
					ClearPathDeleted(ResolvedToPath, bBroadcast);
					BroadcastFileChange(ResolvedToPath, ESandboxFileChange::Removed, bIsEdit ? ESandboxFileChange::Edited : ESandboxFileChange::None);
				}
				else
				{
					BroadcastFileChange(ResolvedToPath, ESandboxFileChange::None, bIsEdit ? ESandboxFileChange::Edited : ESandboxFileChange::Added);
				}
				return true;
			}
			return false;
		}
	}
	else if (ResolvedFromPath.HasSandboxPath())
	{
		// Sandbox -> Non-sandbox
		if (IsPathDeleted(ResolvedFromPath))
		{
			// Cannot move a deleted file
			return false;
		}

		bool bSuccess = false;
		if (LowerLevel->FileExists(*ResolvedFromPath.GetSandboxPath()))
		{
			// Moving an internal sandbox file - can move
			bSuccess = LowerLevel->MoveFile(*ResolvedToPath.GetNonSandboxPath(), *ResolvedFromPath.GetSandboxPath());
		}
		else
		{
			// Moving an external sandbox file - must copy
			bSuccess = LowerLevel->CopyFile(*ResolvedToPath.GetNonSandboxPath(), *ResolvedFromPath.GetNonSandboxPath());
		}

		// ResolvedFromPath may be a file that was added in the sandbox, so the non-sandbox counterpart may not exist.
		// We can get into this path when From = new .uasset, and To = a .temp file (some engine ops create temp, backup files of assets).
		const bool bNeedsMarkForRemove = LowerLevel->FileExists(*ResolvedFromPath.GetNonSandboxPath());
		if (bSuccess && bNeedsMarkForRemove)
		{
			MarkPathDeleted(ResolvedFromPath);
			NotifyFileDeleted(ResolvedFromPath);
		}
		
		return bSuccess;
	}

	// Non-sandbox -> Non-sandbox
	return LowerLevel->MoveFile(*ResolvedToPath.GetNonSandboxPath(), *ResolvedFromPath.GetNonSandboxPath());
}

FDateTime FSandboxPlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return FDateTime::MinValue();
		}

		if (LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->GetTimeStamp(*ResolvedPath.GetSandboxPath());
		}
	}

	return LowerLevel->GetTimeStamp(*ResolvedPath.GetNonSandboxPath());
}

void FSandboxPlatformFile::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return;
		}

		if (LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->SetTimeStamp(*ResolvedPath.GetSandboxPath(), DateTime);
		}

		return; // Do not attempt to modify the non-sandbox file
	}

	return LowerLevel->SetTimeStamp(*ResolvedPath.GetNonSandboxPath(), DateTime);
}

FDateTime FSandboxPlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return FDateTime::MinValue();
		}

		if (LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->GetAccessTimeStamp(*ResolvedPath.GetSandboxPath());
		}
	}

	return LowerLevel->GetAccessTimeStamp(*ResolvedPath.GetNonSandboxPath());
}

FString FSandboxPlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);
	return LowerLevel->GetFilenameOnDisk(*ResolvedPath.GetNonSandboxPath());
}

IFileHandle* FSandboxPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return nullptr;
		}

		if (bAllowWrite)
		{
			MigrateFileToSandbox(ResolvedPath);
		}

		if (bAllowWrite || LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->OpenRead(*ResolvedPath.GetSandboxPath(), bAllowWrite);
		}
	}

	return LowerLevel->OpenRead(*ResolvedPath.GetNonSandboxPath(), bAllowWrite);
}

IFileHandle* FSandboxPlatformFile::OpenReadNoBuffering(const TCHAR* Filename, bool bAllowWrite)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return nullptr;
		}

		if (bAllowWrite)
		{
			MigrateFileToSandbox(ResolvedPath);
		}

		if (bAllowWrite || LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->OpenReadNoBuffering(*ResolvedPath.GetSandboxPath(), bAllowWrite);
		}
	}

	return LowerLevel->OpenReadNoBuffering(*ResolvedPath.GetNonSandboxPath(), bAllowWrite);
}

IFileHandle* FSandboxPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		MigrateFileToSandbox(ResolvedPath);

		IFileHandle* Handle = LowerLevel->OpenWrite(*ResolvedPath.GetSandboxPath(), bAppend, bAllowRead);
		if (Handle)
		{
			ClearPathDeleted(ResolvedPath);
		}
		return Handle;
	}

	return LowerLevel->OpenWrite(*ResolvedPath.GetNonSandboxPath(), bAppend, bAllowRead);
}

bool FSandboxPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Directory);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return false;
		}

		if (LowerLevel->DirectoryExists(*ResolvedPath.GetSandboxPath()))
		{
			return true;
		}
	}

	return LowerLevel->DirectoryExists(*ResolvedPath.GetNonSandboxPath());
}

bool FSandboxPlatformFile::CreateDirectory(const TCHAR* Directory)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Directory);

	if (ResolvedPath.HasSandboxPath())
	{
		if (LowerLevel->CreateDirectory(*ResolvedPath.GetSandboxPath()))
		{
			ClearPathDeleted(ResolvedPath);
			return true;
		}

		return false; // Do not attempt to create the non-sandbox directory
	}

	return LowerLevel->CreateDirectory(*ResolvedPath.GetNonSandboxPath());
}

bool FSandboxPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Directory);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return true;
		}

		// Iterate the directory to validate that it is really empty before deleting it
		const TArray<FDirectoryItem> DirectoryItems = GetDirectoryContents(ResolvedPath, Directory);
		if (DirectoryItems.Num() == 0 && (!LowerLevel->DirectoryExists(*ResolvedPath.GetSandboxPath()) || LowerLevel->DeleteDirectory(*ResolvedPath.GetSandboxPath())))
		{
			MarkPathDeleted(ResolvedPath);
			return true;
		}

		return false; // Do not attempt to create the non-sandbox directory
	}

	return LowerLevel->DeleteDirectory(*ResolvedPath.GetNonSandboxPath());
}

FFileStatData FSandboxPlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(FilenameOrDirectory);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return FFileStatData();
		}

		if (LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->GetStatData(*ResolvedPath.GetSandboxPath());
		}

		FFileStatData StatData = LowerLevel->GetStatData(*ResolvedPath.GetNonSandboxPath());
		// Sandbox files are always writeable.
		StatData.bIsReadOnly = false;
		return StatData;
	}

	return LowerLevel->GetStatData(*ResolvedPath.GetNonSandboxPath());
}

bool FSandboxPlatformFile::IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Directory);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return false;
		}

		const TArray<FDirectoryItem> DirectoryItems = GetDirectoryContents(ResolvedPath, Directory);
		for (const FDirectoryItem& DirectoryItem : DirectoryItems)
		{
			if (!Visitor.CallShouldVisitAndVisit(*DirectoryItem.Path, DirectoryItem.StatData.bIsDirectory))
			{
				return false;
			}
		}

		return true;
	}

	return LowerLevel->IterateDirectory(Directory, Visitor); // Note: Using the path we were given here to ensure the calling code gets the expected path
}

bool FSandboxPlatformFile::IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Directory);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath))
		{
			return false;
		}

		const TArray<FDirectoryItem> DirectoryItems = GetDirectoryContents(ResolvedPath, Directory);
		for (const FDirectoryItem& DirectoryItem : DirectoryItems)
		{
			if (!Visitor.CallShouldVisitAndVisit(*DirectoryItem.Path, DirectoryItem.StatData))
			{
				return false;
			}
		}

		return true;
	}

	return LowerLevel->IterateDirectoryStat(Directory, Visitor); // Note: Using the path we were given here to ensure the calling code gets the expected path
}

IAsyncReadFileHandle* FSandboxPlatformFile::OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite /*= false*/)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath) || LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->OpenAsyncRead(*ResolvedPath.GetSandboxPath(), bAllowWrite);
		}
	}

	return LowerLevel->OpenAsyncRead(*ResolvedPath.GetNonSandboxPath(), bAllowWrite);
}

void FSandboxPlatformFile::SetAsyncMinimumPriority(EAsyncIOPriorityAndFlags MinPriority)
{
	LowerLevel->SetAsyncMinimumPriority(MinPriority);
}

FString FSandboxPlatformFile::ConvertToAbsolutePathForExternalAppForRead(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		if (IsPathDeleted(ResolvedPath) || LowerLevel->FileExists(*ResolvedPath.GetSandboxPath()))
		{
			return LowerLevel->ConvertToAbsolutePathForExternalAppForRead(*ResolvedPath.GetSandboxPath());
		}
	}

	return LowerLevel->ConvertToAbsolutePathForExternalAppForRead(*ResolvedPath.GetNonSandboxPath());
}

FString FSandboxPlatformFile::ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename)
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(Filename);

	if (ResolvedPath.HasSandboxPath())
	{
		return LowerLevel->ConvertToAbsolutePathForExternalAppForWrite(*ResolvedPath.GetSandboxPath());
	}

	return LowerLevel->ConvertToAbsolutePathForExternalAppForWrite(*ResolvedPath.GetNonSandboxPath());
}

ESandboxFileChange FSandboxPlatformFile::PersistItem(
	const FSandboxedPlatformFilePath& InFile,
	ISourceControlProvider& InSourceControlProvider, const FPersistArgs& InParams, IPersistFeedback& InErrorHandler
	)
{
	if (IsPathDeleted(InFile))
	{
		return DeleteFileWithSCC(*LowerLevel, InSourceControlProvider, InErrorHandler, *InFile.GetNonSandboxPath())
			? ESandboxFileChange::Removed
			: ESandboxFileChange::None; 
	}
	
	const TCHAR* InToFilename = *InFile.GetNonSandboxPath();
	const bool bExistsInNonSandbox = LowerLevel->FileExists(InToFilename);
	const bool bIsAddOp = !bExistsInNonSandbox;
	if (bIsAddOp)
	{
		return AddFileWithSCC(*LowerLevel, InSourceControlProvider, InErrorHandler, InToFilename, *InFile.GetSandboxPath())
			? ESandboxFileChange::Added 
			: ESandboxFileChange::None;
	}
	return EditFileWithSCC(*LowerLevel, InSourceControlProvider, InErrorHandler, InToFilename, *InFile.GetSandboxPath(), InParams.bShouldMakeWritableIfNoSourceControl)
		? ESandboxFileChange::Edited 
		: ESandboxFileChange::None;
}

bool FSandboxPlatformFile::PersistSandbox(
	ISourceControlProvider& InSourceControlProvider,
	const FPersistArgs& InParams, 
	TFunctionRef<void(const FSandboxedPlatformFilePath&, ESandboxFileChange)> InOnPersistFile,
	IPersistFeedback& InPersistFeedback
	)
{
	// We need to disable the sandbox while we do this
	bool bFullOperationSuccess = true;
	TGuardValue<TAtomic<bool>, bool> DisableSandboxGuard(bSandboxEnabled, false);
	
	FScopedRegisterExternalFileChanges FileChanges;
	for (const FString& File : InParams.Files)
	{
		InPersistFeedback.StartFile(*File);

		const FSandboxedPlatformFilePath FilePath = ToSandboxPath(File, true);
		const ESandboxFileChange ActionPerformed = PersistItem(FilePath, InSourceControlProvider, InParams, InPersistFeedback);
		const bool bSuccess = ActionPerformed != ESandboxFileChange::None;

		// if the file operation was successful, mark the file as persisted
		if (bSuccess)
		{
			InPersistFeedback.HandleSuccess(*File);
			InOnPersistFile(FilePath, ActionPerformed);

			// Persisting an Added/Edited file moves it from the sandbox path to the non-sandbox path and detaches the package's linker.
			// Without notifying, the Asset Registry can see only the sandbox-side delete and drop the asset from the Content Browser.
			RegisterAddedOrEditedOnPersist(FileChanges, ActionPerformed, FilePath);
		}

		bFullOperationSuccess &= bSuccess;
	}

	return bFullOperationSuccess;
}

void FSandboxPlatformFile::DiscardAll(
	TArray<FName>& OutPackagesPendingHotReload, TArray<FName>& OutPackagesPendingPurge,
	TFunctionRef<void(const FFileChange&)> InProcessFileChangeCallback
	)
{
	DiscardSandboxInternal(
		OutPackagesPendingHotReload, OutPackagesPendingPurge, 
		InProcessFileChangeCallback, 
		[&InProcessFileChangeCallback](const FSandboxedPlatformFilePath& Path, EFileChangeAction Action)
		{
			InProcessFileChangeCallback(FFileChange(Path.GetNonSandboxPath(), Action));
		});
}

void FSandboxPlatformFile::RevertSandbox(
	TConstArrayView<FString> InFilesToRevert, 
	TArray<FName>& OutPackagesPendingHotReload, TArray<FName>& OutPackagesPendingPurge, 
	TFunctionRef<void(const FFileChange&)> InProcessFileChangeCallback
	)
{
	// Additionally, also delete from the sandbox internal file system, so the change is not rediscovered when reloading the sandbox.
	const auto HandleRevertAddedOrEditedFile =[this, &InProcessFileChangeCallback](const FSandboxedPlatformFilePath& Path, EFileChangeAction Action)
	{
		const FString& SandboxPath = Path.GetSandboxPath();
		if (ensure(LowerLevel->FileExists(*SandboxPath)))
		{
			LowerLevel->DeleteFile(*SandboxPath);
		}
			
		InProcessFileChangeCallback(FFileChange(Path.GetNonSandboxPath(), Action));
	};
	
	DiscardSandboxInternal(
		OutPackagesPendingHotReload, OutPackagesPendingPurge, 
		InProcessFileChangeCallback, HandleRevertAddedOrEditedFile,
		InFilesToRevert
		);
}

bool FSandboxPlatformFile::DeletedPackageExistsInNonSandbox(FString InFilename) const
{
	const FSandboxedPlatformFilePath ResolvedPath = ToSandboxPath(MoveTemp(InFilename));
	return ResolvedPath.HasSandboxPath()
		&& IsPathDeleted(ResolvedPath)
		&& LowerLevel->FileExists(*ResolvedPath.GetNonSandboxPath());
}

void FSandboxPlatformFile::DiscardSandboxInternal(
	TArray<FName>& OutPackagesPendingHotReload, TArray<FName>& OutPackagesPendingPurge, 
	TFunctionRef<void(const FFileChange&)> InProcessRemoved,
	TFunctionRef<void(const FSandboxedPlatformFilePath& Path, EFileChangeAction Action)> InProcessAddedOrEdited,
	TConstArrayView<FString> InFilesToDiscard
	)
{
	// We need to disable the sandbox while we do this
	const TGuardValue<TAtomic<bool>, bool> DisableSandboxGuard(bSandboxEnabled, false);
	
	// At the end, notify the Asset Registry (AR) / Content Browser that the sandboxed directories have been restored to their original state.
	FScopedRegisterExternalFileChanges FileChanges;
	
	// Add any files that were deleted by the sandbox but exist in the non-sandbox directory
	DiscardRemoved(InFilesToDiscard, OutPackagesPendingPurge, [&](const FString& NonSandboxPath)
	{
		const FFileChange Change(NonSandboxPath, EFileChangeAction::Added);
		FileChanges.RegisterFileChange(Change);
		InProcessRemoved(Change);
	});

	{
		const FScopeLock SandboxMountPointsLock(&SandboxMountPointsCS);
		DiscardAddedAndModifiedPaths(
			InFilesToDiscard, SandboxMountPoints, *LowerLevel, OutPackagesPendingHotReload, OutPackagesPendingPurge, 
			[&](const FSandboxedPlatformFilePath& Path, EFileChangeAction Action)
			{
				FileChanges.RegisterFileChange(FFileChange(Path.GetNonSandboxPath(), Action));
				InProcessAddedOrEdited(Path, Action);
			});
	}
}

void FSandboxPlatformFile::DiscardRemoved(
	TConstArrayView<FString> InFilesToDiscard, 
	TArray<FName>& OutPackagesPendingPurge, 
	TFunctionRef<void(const FString&)> InProcessRemoved
	)
{
	const FScopeLock DeletedSandboxPathsLock(&DeletedSandboxPathsCS);
	const auto DiscardSingle = [&OutPackagesPendingPurge, &InProcessRemoved](const FString& NonSandboxPath)
	{
		// If this file maps to a package then we need to flush its linker so that we can remove the file from the sandbox
		FName PackageName;
		FlushPackageFile(NonSandboxPath, &PackageName);
    
		if (!PackageName.IsNone())
		{
			OutPackagesPendingPurge.Add(PackageName);
		}
    
		InProcessRemoved(NonSandboxPath);
	};
	
	const bool bClearAll = InFilesToDiscard.IsEmpty();
	if (bClearAll)
	{
		for (const FString& NonSandboxPath : DeletedSandboxPaths)
		{
			DiscardSingle(NonSandboxPath);
		}
		DeletedSandboxPaths.Empty();
	}
	else
	{
		for (const FString& NonSandboxPath : InFilesToDiscard)
		{
			const FString AbsPath = FPaths::ConvertRelativePathToFull(NonSandboxPath);
			const bool bIsMarkedRemoved = DeletedSandboxPaths.Contains(AbsPath);
			if (bIsMarkedRemoved)
			{
				DiscardSingle(AbsPath);
				DeletedSandboxPaths.Remove(AbsPath);
			}
		}
	}
}

void FSandboxPlatformFile::OnContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath)
{
	RegisterContentMountPath(InAssetPath, InFilesystemPath);
}

void FSandboxPlatformFile::OnContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath)
{
	UnregisterContentMountPath(InAssetPath, InFilesystemPath);
}

void FSandboxPlatformFile::RegisterContentMountPath(const FString& InAssetPath, const FString& InFilesystemPath)
{
	const FSandboxedPlatformFilePath MountPointPath = FSandboxedPlatformFilePath::CreateMountPoint(
		GetSandboxRootPath(), InAssetPath, InFilesystemPath
		);
	const bool bCreatedDirectory = LowerLevel->CreateDirectory(*MountPointPath.GetSandboxPath());
	UE_CLOGF(!bCreatedDirectory, LogFileSandbox, Error,
		"Failed to create sandboxed mount directory %ls for mount point %ls",
		*MountPointPath.GetSandboxPath(), *MountPointPath.GetNonSandboxPath()
		);
	
	if (bCreatedDirectory)
	{
		const FScopeLock SandboxMountPointsLock(&SandboxMountPointsCS);

		FSandboxMountPoint& SandboxMountPoint = SandboxMountPoints.Add_GetRef(FSandboxMountPoint{ MountPointPath });
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
		if (IDirectoryWatcher* DirectoryWatcher = GetDirectoryWatcher())
		{
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
				SandboxMountPoint.Path.GetSandboxPath(), 
				IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FSandboxPlatformFile::OnSandboxDirectoryChanged, SandboxMountPoint.Path), 
				SandboxMountPoint.OnDirectoryChangedHandle, IDirectoryWatcher::IncludeDirectoryChanges
				);
		}
#endif
	}
}

void FSandboxPlatformFile::UnregisterContentMountPath(const FString& InAssetPath, const FString& InFilesystemPath)
{
	const FString AbsoluteSandboxPath = FPaths::ConvertRelativePathToFull(GetSandboxRootPath() / InAssetPath) / TEXT("");
	{
		const FScopeLock DeletedSandboxPathsLock(&DeletedSandboxPathsCS);
		for (auto It = DeletedSandboxPaths.CreateIterator(); It; ++It)
		{
			const FSandboxedPlatformFilePath SandboxPath = ToSandboxPath_Absolute(*It);
			if (SandboxPath.GetSandboxPath().StartsWith(AbsoluteSandboxPath))
			{
				It.RemoveCurrent();
				continue;
			}
		}
	}
	
	{
		const FScopeLock SandboxMountPointsLock(&SandboxMountPointsCS);
		SandboxMountPoints.RemoveAll([&AbsoluteSandboxPath](FSandboxMountPoint& InSandboxMountPoint) -> bool
		{
			const bool bShouldRemove = InSandboxMountPoint.Path.GetSandboxPath() == AbsoluteSandboxPath;
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
			if (bShouldRemove && InSandboxMountPoint.OnDirectoryChangedHandle.IsValid())
			{
				if (IDirectoryWatcher* DirectoryWatcher = GetDirectoryWatcherIfLoaded())
				{
					DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(InSandboxMountPoint.Path.GetSandboxPath(), InSandboxMountPoint.OnDirectoryChangedHandle);
					InSandboxMountPoint.OnDirectoryChangedHandle.Reset();
				}
			}
#endif
			return bShouldRemove;
		});
	}
	LowerLevel->DeleteDirectoryRecursively(*AbsoluteSandboxPath);
}

FSandboxedPlatformFilePath FSandboxPlatformFile::ToSandboxPath(FString InFilename, const bool bEvenIfDisabled) const
{
	return ToSandboxPath_Absolute(FPaths::ConvertRelativePathToFull(MoveTemp(InFilename)), bEvenIfDisabled);
}

FSandboxedPlatformFilePath FSandboxPlatformFile::ToSandboxPath_Absolute(FString InFilename, const bool bEvenIfDisabled) const
{
	if (bEvenIfDisabled || IsSandboxEnabled())
	{
		FScopeLock SandboxMountPointsLock(&SandboxMountPointsCS);
		for (const FSandboxMountPoint& SandboxMountPoint : SandboxMountPoints)
		{
			if (const TOptional<FSandboxedPlatformFilePath> Path = MakeSandboxedFilePathIfInMountPoint(InFilename, SandboxMountPoint.Path))
			{
				return *Path;
			}
		}
	}

	return FSandboxedPlatformFilePath(MoveTemp(InFilename));
}

FSandboxedPlatformFilePath FSandboxPlatformFile::FromSandboxPath(FString InFilename) const
{
	return FromSandboxPath_Absolute(FPaths::ConvertRelativePathToFull(MoveTemp(InFilename)));
}

FSandboxedPlatformFilePath FSandboxPlatformFile::FromSandboxPath_Absolute(FString InFilename) const
{
	FScopeLock SandboxMountPointsLock(&SandboxMountPointsCS);

	for (const FSandboxMountPoint& SandboxMountPoint : SandboxMountPoints)
	{
		// Mount point are stored with a trailing slash to prevent matching mount point with similar names -> (/Bla/Content, /Bla/ContentSupreme)
		// So we test without the slash to make sure with can match mount point directly -> (/Bla/Content match /Bla/Content/)
		const FString& PathStr = SandboxMountPoint.Path.GetSandboxPath();
		int32 PathStrNoSlashLength = PathStr.Len() - 1;
		if (FCString::Strnicmp(*InFilename, *PathStr, PathStrNoSlashLength) == 0 && (InFilename.Len() == PathStrNoSlashLength || InFilename[PathStrNoSlashLength] == TEXT('/')))
		{
			return FSandboxedPlatformFilePath::CreateNonSandboxPath(MoveTemp(InFilename), SandboxMountPoint.Path);
		}
	}

	return FSandboxedPlatformFilePath(MoveTemp(InFilename));
}

bool FSandboxPlatformFile::IsPathDeleted(const FSandboxedPlatformFilePath& InPath) const
{
	FScopeLock DeletedSandboxPathsLock(&DeletedSandboxPathsCS);
	return DeletedSandboxPaths.Contains(InPath.GetNonSandboxPath());
}

void FSandboxPlatformFile::MarkPathDeleted(const FSandboxedPlatformFilePath& InPath)
{
	const FNonSandboxPath& NonSandboxPath = InPath.GetNonSandboxPath();
	const bool bIsNonSandbox = LowerLevel->FileExists(*NonSandboxPath) || LowerLevel->DirectoryExists(*NonSandboxPath);
	// If the file or directory does not exist, it cannot be marked as deleted. This ensure is a sanity check that our logic is correct.
	if (!ensureAlways(bIsNonSandbox))
	{
		return;
	}
	
	const bool bMadeChanges = [this, &InPath]
	{
		const FScopeLock DeletedSandboxPathsLock(&DeletedSandboxPathsCS);
		bool bAlreadyInSet = false;
		DeletedSandboxPaths.Add(InPath.GetNonSandboxPath(), &bAlreadyInSet);
			
		const bool bMadeChanges = !bAlreadyInSet;
		return bMadeChanges;
	}();
	
	if (bMadeChanges)
	{
		BroadcastFileChange(InPath, ESandboxFileChange::None, ESandboxFileChange::Removed);
	}
}

void FSandboxPlatformFile::ClearPathDeleted(const FSandboxedPlatformFilePath& InPath, bool bBroadcast)
{
	const bool bMadeChanges = [this, &InPath]
	{
		const FScopeLock DeletedSandboxPathsLock(&DeletedSandboxPathsCS);
		const int32 NumRemoved = DeletedSandboxPaths.Remove(InPath.GetNonSandboxPath());
		return NumRemoved > 0;
	}();
	
	if (bMadeChanges && bBroadcast)
	{
		BroadcastFileChange(InPath, ESandboxFileChange::Removed, ESandboxFileChange::None);
	}
}

void FSandboxPlatformFile::NotifyFileDeleted(const FSandboxedPlatformFilePath& InPath)
{
	if (!IsSandboxEnabled())
	{
		return;
	}

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
	if (FDirectoryWatcherModule* DirectoryWatcherModule = GetDirectoryWatcherModuleIfLoaded())
	{
		FFileChangeData FileChange(InPath.GetNonSandboxPath(), FFileChangeData::FCA_Removed);
		DirectoryWatcherModule->RegisterExternalChanges(TArrayView<const FFileChangeData>(&FileChange, 1));
	}
#endif
}

void FSandboxPlatformFile::MigrateFileToSandbox(const FSandboxedPlatformFilePath& InPath) const
{
	checkf(InPath.HasSandboxPath(), TEXT("MigrateFileToSandbox requires a sandbox path to be set!"));

	// Migrate the non-sandbox directory structure to the sandbox
	{
		const FString SandboxDirectoryPath = FPaths::GetPath(InPath.GetSandboxPath());

		// We create the directory if no part of it has been explicitly deleted in this sandbox
		bool bCreateDirectory = true;
		{
			// Walk the paths backwards for as long as they match (which is the sandbox relative part of the paths)
			FSandboxedPlatformFilePath TmpSandboxFilePath(FPaths::GetPath(InPath.GetNonSandboxPath()), CopyTemp(SandboxDirectoryPath));
			while (FPaths::GetBaseFilename(TmpSandboxFilePath.GetNonSandboxPath()) == FPaths::GetBaseFilename(TmpSandboxFilePath.GetSandboxPath()))
			{
				if (IsPathDeleted(TmpSandboxFilePath))
				{
					bCreateDirectory = false;
					break;
				}
				TmpSandboxFilePath = FSandboxedPlatformFilePath(FPaths::GetPath(TmpSandboxFilePath.GetNonSandboxPath()), FPaths::GetPath(TmpSandboxFilePath.GetSandboxPath()));
			}
		}
		if (bCreateDirectory)
		{
			LowerLevel->CreateDirectoryTree(*SandboxDirectoryPath);
		}
	}

	if (IsPathDeleted(InPath))
	{
		// Sandbox has explicitly deleted this file - don't resurrect it from the non-sandbox file
		return;
	}

	if (LowerLevel->FileExists(*InPath.GetSandboxPath()))
	{
		UE_LOGF(LogFileSandbox, Verbose, "MigrateFileToSandbox: Sandbox has the file %ls already at this location. Nothing to do.", *InPath.GetSandboxPath() );
		// Sandbox already has a file at this location - nothing to do
		return;
	}

	if (!LowerLevel->FileExists(*InPath.GetNonSandboxPath()))
	{
		UE_LOGF(LogFileSandbox, Verbose, "MigrateFileToSandbox: There is no file %ls at this location. Nothing to do.", *InPath.GetNonSandboxPath() );
		// Non-sandbox has no file at this location - nothing to do
		return;
	}

	UE_LOGF(LogFileSandbox, Verbose, "MigrateFileToSandbox: Copying file %ls into the sandbox.", *InPath.GetNonSandboxPath() );
	// Copy the file into the sandbox
	LowerLevel->CopyFile(*InPath.GetSandboxPath(), *InPath.GetNonSandboxPath());

	// Ensure the migrated file is writable
	LowerLevel->SetReadOnly(*InPath.GetSandboxPath(), false);
}

TArray<FSandboxPlatformFile::FDirectoryItem> FSandboxPlatformFile::GetDirectoryContents(const FSandboxedPlatformFilePath& InPath, const TCHAR* InDirBase) const
{
	checkf(InPath.HasSandboxPath(), TEXT("GetDirectoryContents requires a sandbox path to be set!"));
	TMap<FString, FFileStatData> FoundItems;

	// Gather the items, the sandbox iteration is straightforward
	LowerLevel->IterateDirectoryStat(*InPath.GetSandboxPath(), [&FoundItems](const TCHAR* FilenameOrDirectory, const FFileStatData& StatData)
	{
		FoundItems.Add(FPaths::GetCleanFilename(FilenameOrDirectory), StatData);
		return true;
	});
	// Gather the non-sandbox, validating we haven't already gathered the sandbox equivalent or the file/dir is marked as deleted
	LowerLevel->IterateDirectoryStat(*InPath.GetNonSandboxPath(), [this, &InPath, &FoundItems](const TCHAR* FilenameOrDirectory, const FFileStatData& StatData)
	{
		FString SandboxFilenameOrDirectory = FilenameOrDirectory;
		FString CleanFilenameOrDir = FPaths::GetCleanFilename(SandboxFilenameOrDirectory);

		if (!FoundItems.Contains(CleanFilenameOrDir) &&
			!IsPathDeleted(FSandboxedPlatformFilePath::CreateSandboxPath(MoveTemp(SandboxFilenameOrDirectory), InPath)))
		{
			FoundItems.Add(MoveTemp(CleanFilenameOrDir), StatData);
		}
		return true;
	});

	// Turn the found items in an array and re-base on InDirBase
	FString DirBase(InDirBase);
	TArray<FDirectoryItem> DirectoryContents;
	DirectoryContents.Reserve(FoundItems.Num());
	for (const auto& FoundItemPair : FoundItems)
	{
		DirectoryContents.Add(FDirectoryItem{ DirBase / FoundItemPair.Key, FoundItemPair.Value });
	}

	// Sort the result
	DirectoryContents.Sort([](const FDirectoryItem& InOne, const FDirectoryItem& InTwo) -> bool
	{
		return InOne.Path < InTwo.Path;
	});

	return DirectoryContents;
}

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
void FSandboxPlatformFile::OnSandboxDirectoryChanged(const TArray<FFileChangeData>& FileChanges, FSandboxedPlatformFilePath MountPath)
{
	if (!IsSandboxEnabled())
	{
		return;
	}

	if (FDirectoryWatcherModule* DirectoryWatcherModule = GetDirectoryWatcherModuleIfLoaded())
	{
		TArray<FFileChangeData> RemappedFileChanges;
		RemappedFileChanges.Reserve(FileChanges.Num());

		// Map the sandbox paths back to their original roots and notify the directory watcher
		for (const FFileChangeData& FileChange : FileChanges)
		{
			const FSandboxedPlatformFilePath RemappedFilePath = FSandboxedPlatformFilePath::CreateNonSandboxPath(
				FPaths::ConvertRelativePathToFull(FileChange.Filename), MountPath
				);

			// If the sandbox file was removed but the non-sandbox file still exists, this is a
			// persist or edit-revert operation, not a true deletion. Emit FCA_Modified so the
			// asset registry rescans the file rather than removing it from the content browser.
			const FFileChangeData::EFileChangeAction RemappedAction =
				(FileChange.Action == FFileChangeData::FCA_Removed && LowerLevel->FileExists(*RemappedFilePath.GetNonSandboxPath()))
				? FFileChangeData::FCA_Modified
				: FileChange.Action;

			RemappedFileChanges.Emplace(RemappedFilePath.GetNonSandboxPath(), RemappedAction);

			// Make sure the deleted state of this item is synchronized correctly
			if (FileChange.Action == FFileChangeData::FCA_Added)
			{
				ClearPathDeleted(RemappedFilePath);
			}
		}

		DirectoryWatcherModule->RegisterExternalChanges(RemappedFileChanges);
	}
}
#endif


void FSandboxPlatformFile::BroadcastFileChange(const FSandboxedPlatformFilePath& InPath, ESandboxFileChange InOldState, ESandboxFileChange InNewState)
{
	// This would be pointless. Investigate.
	ensure(InOldState != InNewState);
	OnFileStateChangedDelegate.Broadcast(InPath, InOldState, InNewState);
}
}
