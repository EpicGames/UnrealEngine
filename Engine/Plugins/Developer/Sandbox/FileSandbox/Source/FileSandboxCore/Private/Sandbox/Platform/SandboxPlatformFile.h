// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "Delegates/IDelegateInstance.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "SandboxedPlatformFilePath.h"

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
#include "IDirectoryWatcher.h"
#endif

class ISourceControlProvider;

namespace UE::FileSandboxCore
{
class IPersistFeedback;
enum class EFileChangeAction : uint8;
enum class ESandboxFileChange : uint8;
struct FFileChange;
struct FPersistArgs;
struct FPersistResult;
struct FSandboxMountPoint;
	
DECLARE_MULTICAST_DELEGATE_TwoParams(FDeletionStateChanged, const FSandboxedPlatformFilePath&, bool /* bIsDeleted */);
DECLARE_MULTICAST_DELEGATE_OneParam(FSandboxedFilePathDelegate, const FSandboxedPlatformFilePath&);

DECLARE_MULTICAST_DELEGATE_ThreeParams(FFileStateChangeDelegate, const FSandboxedPlatformFilePath&, ESandboxFileChange /*OldState*/, ESandboxFileChange /*NewState*/);
	
class FSandboxPlatformFile : public IPlatformFile
{
public:

	explicit FSandboxPlatformFile(const FString& InSandboxRootPath);
	virtual ~FSandboxPlatformFile() override;
	
	//~ For visibility of overloads we don't override
	using IPlatformFile::IterateDirectory;
	using IPlatformFile::IterateDirectoryStat;
	
	//~ begin IPlatformFile overrides
	virtual void SetSandboxEnabled(bool bInEnabled) override;
	virtual bool IsSandboxEnabled() const override;
	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override;
	virtual void Tick() override;
	virtual IPlatformFile* GetLowerLevel() override;
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override;
	virtual const TCHAR* GetName() const override;
	virtual bool FileExists(const TCHAR* Filename) override;
	virtual int64 FileSize(const TCHAR* Filename) override;
	virtual bool DeleteFile(const TCHAR* Filename) override;
	virtual bool IsReadOnly(const TCHAR* Filename) override;
	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;
	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override;
	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;
	virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override;
	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;
	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual IFileHandle* OpenReadNoBuffering(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;
	virtual bool DirectoryExists(const TCHAR* Directory) override;
	virtual bool CreateDirectory(const TCHAR* Directory) override;
	virtual bool DeleteDirectory(const TCHAR* Directory) override;
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;
	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override;
	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override;
	virtual IAsyncReadFileHandle* OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual void SetAsyncMinimumPriority(EAsyncIOPriorityAndFlags MinPriority) override;
	virtual FString ConvertToAbsolutePathForExternalAppForRead(const TCHAR* Filename) override;
	virtual FString ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename) override;
	//~ end IPlatformFile overrides

	/**
	 *	Persist the file list from the sandbox state onto the real files Will mark files for which the operation was
	 *	successful as persisted.
	 *	@param InSourceControlProvider The source control provider to register the changes with
	 *	@param InParams	Parameters controlling the persist options specified by the user.
	 *	@param InOnPersistFile Invoked for each file successfully persisted.
	 *	@param InPersistFeedback Handles any errors that occur during persist
	 *	@return Whether the persist operation succeeded completely. False if there were errors.
	 */
	bool PersistSandbox(
		ISourceControlProvider& InSourceControlProvider, const FPersistArgs& InParams, 
		TFunctionRef<void(const FSandboxedPlatformFilePath&, ESandboxFileChange)> InOnPersistFile, 
		IPersistFeedback& InPersistFeedback
		);

	/** 
	 * Brings all files back to the state they had before the sandbox:
	 * - notifies the asset registry of the file change
	 * - unloads any FLinker instances that may depend on it
	 * - you'll have to manually hot-reload and purge packages.
	 * This function is useful if the user is leaving the sandbox so when the sandbox is reloaded removed, added & edited files are retained.
	 * 
	 * @see RevertSandbox: The difference to RevertSandbox is that added and edited files remain in the sandbox-internal file system, 
	 * e.g. Intermediate/Sandboxes. This means the next time this sandbox is reloaded, the changes can be discovered.
	 * 
	 * @param OutPackagesPendingHotReload Packages that need to be hot reloaded.
	 * @param OutPackagesPendingPurge Packages that need to be purged.
	 * @param InProcessFileChangeCallback Invoked on non-sandbox paths, and with the file action that reverts the sandbox action. 
	 *	E.g. a file removed in sandbox would be EFileChangeAction::Added.
	 */
	void DiscardAll(
		TArray<FName>& OutPackagesPendingHotReload, TArray<FName>& OutPackagesPendingPurge,
		TFunctionRef<void(const FFileChange&)> InProcessFileChangeCallback = [](const FFileChange&){}
		);
	
	/** 
	 * Brings all files back to the state they had before the sandbox:
	 * - notifies the asset registry of the file change
	 * - unloads any FLinker instances that may depend on it
	 * - you'll have to manually hot-reload and purge packages.
	 * This function is useful if the user remains in the sandbox and wants to undo a file action.
	 * 
	 * @see DiscardAll: The difference to DiscardAll is that added and edited files are removed from the sandbox-internal file system, 
	 * e.g. Intermediate/Sandboxes. This means the next time this sandbox is reloaded, the changes will NOT be discovered.
	 * 
	 * @param InFilesToRevert Absolute or relative non-sandbox paths. If empty, discard all changes.
	 * @param OutPackagesPendingHotReload Packages that need to be hot reloaded.
	 * @param OutPackagesPendingPurge Packages that need to be purged.
	 * @param InProcessFileChangeCallback Invoked on non-sandbox paths, and with the file action that reverts the sandbox action. 
	 *	E.g. a file removed in sandbox would be EFileChangeAction::Added.
	 */
	void RevertSandbox(
		TConstArrayView<FString> InFilesToRevert,
		TArray<FName>& OutPackagesPendingHotReload, TArray<FName>& OutPackagesPendingPurge,
		TFunctionRef<void(const FFileChange&)> InProcessFileChangeCallback = [](const FFileChange&){}
		);
	
	/** @return Whether the given package file was deleted and exists on non sandbox path. */
	bool DeletedPackageExistsInNonSandbox(FString InFilename) const;

	/** Invoked when the file action of a file changes. */
	FFileStateChangeDelegate& OnFileStateChanged() { return OnFileStateChangedDelegate; }
	
private:
	
	struct FDirectoryItem
	{
		FString Path;
		FFileStatData StatData;
	};

	/**
	 * Brings all files back to the state they had before the sandbox:
	 * - notifies the asset registry of the file change
	 * - unloads any FLinker instances that may depend on it
	 * - you'll have to manually hot-reload and purge packages.
	 * 
	 * @param OutPackagesPendingHotReload Packages that need to be hot reloaded.
	 * @param OutPackagesPendingPurge Packages that need to be purged.
	 * @param InProcessRemoved Invoked on non-sandbox paths, and with the file action that reverts the sandbox action. 
	 * @param InProcessAddedOrEdited Invoked with the file action that reverts the sandbox action. 
	 * @param InFilesToDiscard Absolute or relative non-sandbox paths to discard. If empty, discard all files.
	 */
	void DiscardSandboxInternal(
		TArray<FName>& OutPackagesPendingHotReload, TArray<FName>& OutPackagesPendingPurge,
		TFunctionRef<void(const FFileChange&)> InProcessRemoved = [](const FFileChange&){},
		TFunctionRef<void(const FSandboxedPlatformFilePath& Path, EFileChangeAction Action)> InProcessAddedOrEdited = [](const FSandboxedPlatformFilePath&, EFileChangeAction){},
		TConstArrayView<FString> InFilesToDiscard = {}
		);
	
	/** Utility for discarding files marked as removed. */
	void DiscardRemoved(
		TConstArrayView<FString> InFilesToDiscard,
		TArray<FName>& OutPackagesPendingPurge,
		TFunctionRef<void(const FString&)> InProcessRemoved = [](const FString&){}
		);
	
	/** Callback when a new content path is mounted */
	void OnContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath);
	/** Callback when an existing content path is unmounted */
	void OnContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath);

	/** Register a mount path from a source content path */
	void RegisterContentMountPath(const FString& InAssetPath, const FString& InFilesystemPath);
	/** Unregister a mount path from a source content path */
	void UnregisterContentMountPath(const FString& InAssetPath, const FString& InFilesystemPath);

	/** Resolve the given path to its sandbox path (if any) */
	FSandboxedPlatformFilePath ToSandboxPath(FString InFilename, const bool bEvenIfDisabled = false) const;
	/** Resolve the given path to its sandbox path (if any) from an absolute filename */
	FSandboxedPlatformFilePath ToSandboxPath_Absolute(FString InFilename, const bool bEvenIfDisabled = false) const;
	/** Resolve the given path to its non-sandbox path (if any) */
	FSandboxedPlatformFilePath FromSandboxPath(FString InFilename) const;
	/** Resolve the given path to its non-sandbox path (if any) from an absolute filename */
	FSandboxedPlatformFilePath FromSandboxPath_Absolute(FString InFilename) const;

	/** @return Whether a file or directory that existed in the non-sandbox has been explicitly deleted by the sandbox. */
	bool IsPathDeleted(const FSandboxedPlatformFilePath& InPath) const;
	/** Set that the non-sandbox file or directory is explicitly deleted by the sandbox. */
	void MarkPathDeleted(const FSandboxedPlatformFilePath& InPath);
	/** Clears the deletion state. The file or directory is no longer explicitly deleted by the sandbox. */
	void ClearPathDeleted(const FSandboxedPlatformFilePath& InPath, bool bBroadcast = true);

	/** Notify that a file has been explicitly deleted from the sandbox */
	void NotifyFileDeleted(const FSandboxedPlatformFilePath& InPath);

	/** Helper function to ensure that a sandbox contains a copy of the non-sandbox file (eg, prior to opening an existing file for writing) - does nothing if the sandbox already has the file, or if there is no non-sandbox file to copy */
	void MigrateFileToSandbox(const FSandboxedPlatformFilePath& InPath) const;

	/** Helper function to get the contents of a directory, taking into account the sandbox state - paths are returned relative to InDirBase */
	TArray<FDirectoryItem> GetDirectoryContents(const FSandboxedPlatformFilePath& InPath, const TCHAR* InDirBase) const;

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
	/** Called when a file in a sandbox directory changes on disk */
	void OnSandboxDirectoryChanged(const TArray<FFileChangeData>& FileChanges, FSandboxedPlatformFilePath MountPath);
#endif
	
	/** Broadcasts OnFileStateChangedDelegate and checks invariants. */
	void BroadcastFileChange(const FSandboxedPlatformFilePath& InPath, ESandboxFileChange InOldState, ESandboxFileChange InNewState);

	/** Attempts to persist the file. */
	ESandboxFileChange PersistItem(
		const FSandboxedPlatformFilePath& InFile, ISourceControlProvider& InSourceControlProvider, 
		const FPersistArgs& InParams, IPersistFeedback& InErrorHandler
		);

	FString GetSandboxRootPath() const;
		
	/** Root path of this sandbox */
	const FString RootPath;

	/** Underlying platform file that we're wrapping */
	IPlatformFile* LowerLevel;

	/** Is this sandbox currently enabled? */
	TAtomic<bool> bSandboxEnabled;

	/** Array of sandbox mount points */
	TArray<FSandboxMountPoint> SandboxMountPoints;
	/** Critical section protecting concurrent access to SandboxMountPoints */
	mutable FCriticalSection SandboxMountPointsCS;

	/**
	 * Set of absolute paths (file or directory) that have been explicitly deleted by the sandbox, 
	 * i.e. when persisting these files should be deleted from the non-sandbox.
	 * 
	 * This contains the absolute path to the non-sandbox file or directory.
	 */
	TSet<FNonSandboxPath> DeletedSandboxPaths;
	/** Critical section protecting concurrent access to DeletedSandboxPaths */
	mutable FCriticalSection DeletedSandboxPathsCS;
	
	/**
	 * Invoked when a file action changes.
	 * 
	 * Examples:
	 * - Adding Foo.uasset: (Foo.uasset, None, Added)
	 * - Reverting the addition of Foo.uasset: (Foo.uasset, Added, None)
	 * 
	 * There should never be any permutation of old = None, new = None.
	 */
	FFileStateChangeDelegate OnFileStateChangedDelegate;
};
}

