// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Paths.h"

namespace UE::FileSandboxCore
{
using FNonSandboxPath = FString;
using FSandboxPath = FString;
	
/** A pair of file paths: a path to the original file and a path to the sandboxed version, if it exists. */
class FSandboxedPlatformFilePath
{
public:
	
	explicit FSandboxedPlatformFilePath(FString&& InNonSandboxPath)
		: SandboxPath()
		, NonSandboxPath(MoveTemp(InNonSandboxPath))
	{
	}

	explicit FSandboxedPlatformFilePath(FString&& InNonSandboxPath, FString&& InSandboxPath)
		: SandboxPath(MoveTemp(InSandboxPath))
		, NonSandboxPath(MoveTemp(InNonSandboxPath))
	{
	}

	/**
	 * Creates a platform file path that maps from an engine mount point to sandboxed mount point
	 * @param InSandboxMountPointPath Directory in the sandbox file structure where mount points are stored, i.e. "MySandbox/Sandbox"
	 * @param InAssetPath Name of the mount point, such as "Game", "Engine", "MyPlugin", etc.
	 * @param InFilesystemPath Root directory such as "MyProject/Content", "Engine/Content", "MyPlugin/Content", etc.
	 */
	static FSandboxedPlatformFilePath CreateMountPoint(
		const FString& InSandboxMountPointPath, const FString& InAssetPath, const FString& InFilesystemPath
		)
	{
		FString AbsoluteSandboxPath = FPaths::ConvertRelativePathToFull(InSandboxMountPointPath / InAssetPath) / TEXT("");
		FString AbsoluteNonSandboxPath = FPaths::ConvertRelativePathToFull(InFilesystemPath) / TEXT("");
		return FSandboxedPlatformFilePath(MoveTemp(AbsoluteNonSandboxPath), MoveTemp(AbsoluteSandboxPath));
	}

	static FSandboxedPlatformFilePath CreateSandboxPath(FString&& InNonSandboxPath, const FSandboxedPlatformFilePath& InRootPath)
	{
		checkf(InRootPath.HasSandboxPath(), TEXT("Root '%s' had no sandbox path set!"), *InRootPath.GetNonSandboxPath());
		return CreateSandboxPath(MoveTemp(InNonSandboxPath), InRootPath.GetSandboxPath(), InRootPath.GetNonSandboxPath());
	}

	static FSandboxedPlatformFilePath CreateSandboxPath(FString&& InNonSandboxPath, const FString& InRootSandboxPath, const FString& InRootNonSandboxPath)
	{
		// Mount point are stored with a trailing slash to prevent matching mount point with similar names -> (/Bla/Content, /Bla/ContentSupreme)
		// An extra slash is appended here to make sure with can match mount point directly -> (/Bla/Content match /Bla/Content/)
		FString ResolvedSandboxPath = InNonSandboxPath + TEXT("/");
		checkf(ResolvedSandboxPath.StartsWith(InRootNonSandboxPath), TEXT("Path '%s' was not under the root '%s'!"), *InNonSandboxPath, *InRootNonSandboxPath);
		ResolvedSandboxPath.ReplaceInline(*InRootNonSandboxPath, *InRootSandboxPath);
		ResolvedSandboxPath.RemoveAt(ResolvedSandboxPath.Len() - 1, EAllowShrinking::No);
		return FSandboxedPlatformFilePath(MoveTemp(InNonSandboxPath), MoveTemp(ResolvedSandboxPath));
	}

	static FSandboxedPlatformFilePath CreateNonSandboxPath(FString&& InSandboxPath, const FSandboxedPlatformFilePath& InRootPath)
	{
		checkf(InRootPath.HasSandboxPath(), TEXT("Root '%s' had no sandbox path set!"), *InRootPath.GetNonSandboxPath());
		return CreateNonSandboxPath(MoveTemp(InSandboxPath), InRootPath.GetSandboxPath(), InRootPath.GetNonSandboxPath());
	}

	static FSandboxedPlatformFilePath CreateNonSandboxPath(FString&& InSandboxPath, const FString& InRootSandboxPath, const FString& InRootNonSandboxPath)
	{
		// Mount point are stored with a trailing slash to prevent matching mount point with similar names -> (/Bla/Content, /Bla/ContentSupreme)
		// An extra slash is appended here to make sure with can match mount point directly -> (/Bla/Content match /Bla/Content/)
		FString ResolvedNonSandboxPath = InSandboxPath + TEXT("/");
		checkf(ResolvedNonSandboxPath.StartsWith(InRootSandboxPath), TEXT("Path '%s' was not under the root '%s'!"), *InRootSandboxPath, *InRootSandboxPath);
		ResolvedNonSandboxPath.ReplaceInline(*InRootSandboxPath, *InRootNonSandboxPath);
		ResolvedNonSandboxPath.RemoveAt(ResolvedNonSandboxPath.Len() - 1, EAllowShrinking::No);
		return FSandboxedPlatformFilePath(MoveTemp(ResolvedNonSandboxPath), MoveTemp(InSandboxPath));
	}

	/** Copyable */
	FSandboxedPlatformFilePath(const FSandboxedPlatformFilePath&) = default;
	FSandboxedPlatformFilePath& operator=(const FSandboxedPlatformFilePath&) = default;

	/** Movable */
	FSandboxedPlatformFilePath(FSandboxedPlatformFilePath&&) = default;
	FSandboxedPlatformFilePath& operator=(FSandboxedPlatformFilePath&&) = default;

	bool operator==(const FSandboxedPlatformFilePath& Rhs) const
	{
		return GetSandboxPath() == Rhs.GetSandboxPath() && GetNonSandboxPath() == Rhs.GetNonSandboxPath();
	}

	/** Do we have a sandbox path set? */
	bool HasSandboxPath() const
	{
		return SandboxPath.Len() > 0;
	}

	/** Get the absolute sandbox path */
	const FSandboxPath& GetSandboxPath() const
	{
		return SandboxPath;
	}

	/** Get the absolute non-sandbox path */
	const FNonSandboxPath& GetNonSandboxPath() const
	{
		return NonSandboxPath;
	}

	friend uint32 GetTypeHash(const FSandboxedPlatformFilePath& InPath)
	{
		return HashCombine(GetTypeHash(InPath.SandboxPath), GetTypeHash(InPath.NonSandboxPath));
	}

private:
	
	/** Absolute sandbox path */
	FSandboxPath SandboxPath;

	/** Absolute non-sandbox path */
	FNonSandboxPath NonSandboxPath;
};
}
