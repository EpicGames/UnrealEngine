// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxInstance.h"

#include "Interface/DefaultPackageReloadHandler.h"
#include "Interface/Feedback/AggregatePersistFeedback.h"
#include "Misc/CoreDelegates.h"
#include "Misc/PackageName.h"
#include "Persist/DiffUtils.h"
#include "Persist/LogPersistFeedback.h"
#include "Platform/SandboxPlatformFile.h"
#include "Types/Manager/SandboxInitArgs.h"
#include "Types/Sandbox/RevertResult.h"
#include "Types/SandboxFileChange.h"
#include "Types/Package/HotReloadPackageArgs.h"
#include "Types/Package/PurgePackageArgs.h"
#include "UObject/Package.h"
#include "Utils/AssetUtils.h"
#include "Utils/MountPointUtils.h"
#include "Utils/FileChange/FileChange.h"
#include "Utils/PackageSandboxUtils.h"
#include "Utils/SandboxDirectoryUtils.h"
#include "Utils/SandboxFileUtils.h"

namespace UE::FileSandboxCore
{
namespace Private
{
/**
 * Applies file actions resulting from reverting files to the manifest file.
 * E.g. EFileChangeAction::Added implies that a sandboxed remove operation was reverted (so add it to the manifest).
 */
static void AddDiscardedFileChangeInManifest(
	FFileSandboxCore_ManifestData& OutManifestData, const FFileChange& InChange, 
	const FFileSandboxCore_TimedAbsoluteFilePathArray& InDeletedFiles
	)
{
	switch (InChange.Action)
	{
	case EFileChangeAction::Added:
	{
		const TOptional<FDateTime> Time = InDeletedFiles.GetTimestamp(InChange.Filename);
		OutManifestData.DeletedFiles.Add(InChange.Filename, Time.Get(FDateTime::UtcNow()));
		break;
	}
	case EFileChangeAction::Modified: OutManifestData.ModifiedFiles.Add(InChange.Filename); break;
	case EFileChangeAction::Removed: OutManifestData.AddedFiles.Add(InChange.Filename); break;
	default: break;
	}
}

/**
 * Applies file actions resulting from reverting files to the manifest file. 
 * E.g. EFileChangeAction::Added implies that a sandboxed remove operation was reverted (so removed it from the manifest).
 */
static void RemoveRevertedFileChangeFromManifest(FFileSandboxCore_ManifestData& OutManifestData, const FFileChange& InChange)
{
	switch (InChange.Action)
	{
	case EFileChangeAction::Added: OutManifestData.DeletedFiles.Remove(InChange.Filename); break;
	case EFileChangeAction::Modified: OutManifestData.ModifiedFiles.Remove(InChange.Filename); break;
	case EFileChangeAction::Removed: OutManifestData.AddedFiles.Remove(InChange.Filename); break;
	default: break;
	}
}

static void TrackFileChangeInManifest(FFileSandboxCore_ManifestData& OutManifestData, const FSandboxedFileChangeInfo& InChange)
{
	switch (InChange.Action)
	{
	case ESandboxFileChange::Added: OutManifestData.AddedFiles.Add(InChange.Path); break;
	case ESandboxFileChange::Edited: OutManifestData.ModifiedFiles.Add(InChange.Path); break;
	case ESandboxFileChange::Removed:
	{
		FDateTime Timestamp = InChange.Timestamp != FDateTime::MinValue() ? InChange.Timestamp : FDateTime::UtcNow();
		OutManifestData.DeletedFiles.Add(InChange.Path, Timestamp);
		break;
	}
	default: break;
	}
}

/** 
 * Adds packages with in-memory changes. 
 * 
 * These packages must be reverted in order to discard their in-memory changed.
 * Revert effectively happens by doing a hot reload.
 */
static void AppendPackagesWithInMemoryChanges(TArray<FName>& OutPackagesPendingHotReload)
{
	const TArray<UPackage*> DirtyPackages = GetDirtyPackages();
	Algo::TransformIf(DirtyPackages, OutPackagesPendingHotReload, 
		[&](UPackage* Package) { return !OutPackagesPendingHotReload.Contains(Package->GetFName()); },
		[&](UPackage* Package) { return Package->GetFName(); }
		);
}
}
	
TUniquePtr<FSandboxInstance> FSandboxInstance::CreateNewSandbox(
	FString InRootDirectory, const FSandboxInitArgs& InInitArgs, const FFileSandboxCore_SandboxMetaData& InMetaData
	)
{
	FFileSandboxCore_ManifestData ManifestData;
	ManifestData.Initialize();
	
	const bool bSuccess = FileSandboxCore::SaveManifest(ManifestData, *InRootDirectory) && SaveMetaData(InMetaData, InRootDirectory);
	return bSuccess ? MakeUnique<FSandboxInstance>(MoveTemp(InRootDirectory), InInitArgs, ManifestData, InMetaData) : nullptr;
}

TUniquePtr<FSandboxInstance> FSandboxInstance::LoadSandbox(FString InRootDirectory, const FSandboxInitArgs& InInitArgs)
{
	TOptional<FFileSandboxCore_ManifestData> ManifestData = LoadManifest(InRootDirectory);
	TOptional<FFileSandboxCore_SandboxMetaData> MetaData = LoadMetaData(InRootDirectory);
	const bool bSuccess = ManifestData && MetaData;
	if (!bSuccess)
	{
		return nullptr;
	}
	
	MigrateManifestFilePaths(*ManifestData);
	return bSuccess ? MakeUnique<FSandboxInstance>(MoveTemp(InRootDirectory), InInitArgs, MoveTemp(*ManifestData), MoveTemp(*MetaData)) : nullptr;
}

FSandboxInstance::FSandboxInstance(
	FString InRootDirectory, const FSandboxInitArgs& InitArgs,
	FFileSandboxCore_ManifestData InManifestData, FFileSandboxCore_SandboxMetaData InMetaData
	)
	: RootDirectory(FPaths::ConvertRelativePathToFull(MoveTemp(InRootDirectory)))
	, ManifestContent(MoveTemp(InManifestData))
	, MetaData(MoveTemp(InMetaData))
	, PackageReloadHandler(InitArgs.ReloadHandler ? InitArgs.ReloadHandler.ToSharedRef() : MakeShared<FDefaultPackageReloadHandler>())
	, PlatformFile(MakeUnique<FSandboxPlatformFile>(RootDirectory))
	, SourceControlSandbox(*this, MakeConfigFromInitArgs(InitArgs))
{
	PlatformFile->Initialize(&InitArgs.GetPlatformFile(), TEXT(""));
	ReplayChanges(
		*PackageReloadHandler, 
		ManifestContent.DeletedFiles.GetFiles(), ManifestContent.ModifiedFiles.GetFiles(), ManifestContent.AddedFiles.GetFiles()
		);
	
	PlatformFile->OnFileStateChanged().AddRaw(this, &FSandboxInstance::OnFileDeletionStateChanged);
	
	FCoreDelegates::OnEndFrame.AddRaw(this, &FSandboxInstance::OnEndOfFrame);
}
	
FSandboxInstance::~FSandboxInstance()
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	FFileSandboxCore_TimedAbsoluteFilePathArray DeletedFiles = MoveTemp(ManifestContent.DeletedFiles);
	ManifestContent.Empty();
	TArray<FName> PackagesPendingHotReload, PackagesPendingPurge;
	PlatformFile->DiscardAll(PackagesPendingHotReload, PackagesPendingPurge, [this, &DeletedFiles](const FFileChange& Change)
	{
		Private::AddDiscardedFileChangeInManifest(ManifestContent, Change, DeletedFiles); 
	});
	PlatformFile.Reset();

	// Save the data before hot reloading or purging: in case that causes a crash we'll have an update to manifest file. 
	SaveManifest();
	// This just exists to handle e.g. the user having deleted the file manually during the sandbox. Normally, the metadata file should still be there.
	if (!FPaths::FileExists(RootDirectory / GetMetadataFileName()))
	{
		SaveMetaData(MetaData, RootDirectory);
	}

	if (!IsEngineExitRequested())
	{
		Private::AppendPackagesWithInMemoryChanges(PackagesPendingHotReload);
		// In a World Partition level, actor additions live in per-actor external packages and do not dirty
		// the persistent level itself. Force a reload of the persistent level so the WP runtime is rebuilt
		// from disk and any purged in-memory actors are removed from the editor view.
		AppendExternalPersistentLevelForReload(PackagesPendingHotReload, PackagesPendingPurge);

		// Hot reload after unregistering from most delegates to prevent events triggered by hot-reloading (such as asset deleted) to be recorded as transaction.
		PackageReloadHandler->PurgePackages(FPurgePackageArgs(ESandboxPackageReloadPhase::Shutdown, PackagesPendingPurge));
		PackageReloadHandler->HotReloadPackages(FHotReloadPackageArgs(ESandboxPackageReloadPhase::Shutdown, PackagesPendingHotReload));
	}
}

FPersistResult FSandboxInstance::PersistSandbox(const FPersistArgs& InArgs)
{
	bool bManifestNeedsSave = false;
	const auto HandleFileOp = [this, &bManifestNeedsSave](const FSandboxedPlatformFilePath& InPath, ESandboxFileChange InPersistType)
	{
		switch (InPersistType)
		{
		case ESandboxFileChange::Removed:
			bManifestNeedsSave |= ManifestContent.DeletedFiles.Remove(InPath.GetNonSandboxPath());
			break;
		case ESandboxFileChange::Added:
			bManifestNeedsSave |= ManifestContent.AddedFiles.Remove(InPath.GetNonSandboxPath());
			break;
		case ESandboxFileChange::Edited:
			bManifestNeedsSave |= ManifestContent.ModifiedFiles.Remove(InPath.GetNonSandboxPath());
			break;
			
		default: checkNoEntry(); break;
		}
	};
	
	FLogPersistFeedback ErrorLogger;
	FAggregatePersistFeedback Aggregate({ InArgs.Feedback, &ErrorLogger });
	const bool bFullSuccess = PlatformFile->PersistSandbox(
		SourceControlSandbox.GetProxiedSourceControl(), InArgs, HandleFileOp, Aggregate
		);

	if (bManifestNeedsSave)
	{
		SaveManifest();
		
		// Persisting has removed some file states, so inform anyone that cares about it.
		OnSandboxedFilesChangedDelegate.Broadcast();
	}

	return FPersistResult
	{ 
		bFullSuccess ? EPersistStatus::Success : EPersistStatus::Failure 
	};
}

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
FRevertResult FSandboxInstance::RevertAll()
{
	const FRevertResult Result = RevertSandbox(FRevertArgs());
	PackageReloadHandler->PurgePackages(FPurgePackageArgs(ESandboxPackageReloadPhase::Sandboxed, Result.PackagesPendingPurge));
	PackageReloadHandler->HotReloadPackages(FHotReloadPackageArgs(ESandboxPackageReloadPhase::Sandboxed, Result.PackagesPendingHotReload));
	return Result;
}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
FRevertResult FSandboxInstance::RevertSpecified(const TConstArrayView<FString>& InFiles)
{
	const FRevertResult Result = RevertSandbox(FRevertArgs(InFiles));
	PackageReloadHandler->PurgePackages(FPurgePackageArgs(ESandboxPackageReloadPhase::Sandboxed, Result.PackagesPendingPurge));
	PackageReloadHandler->HotReloadPackages(FHotReloadPackageArgs(ESandboxPackageReloadPhase::Sandboxed, Result.PackagesPendingHotReload));
	return Result;
}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
FRevertResult FSandboxInstance::RevertSandbox(const FRevertArgs& InArgs)
{
	FRevertResult Result;
	PlatformFile->RevertSandbox(
		InArgs.FilesToRevert, Result.PackagesPendingHotReload, Result.PackagesPendingPurge, 
		[this](const FFileChange& Change) { Private::RemoveRevertedFileChangeFromManifest(ManifestContent, Change);}
		);
	SaveManifest();
	
	// Discarding has removed all file states, so inform anyone that cares about it.
	OnSandboxedFilesChangedDelegate.Broadcast();

	if (!IsEngineExitRequested())
	{
		Private::AppendPackagesWithInMemoryChanges(Result.PackagesPendingHotReload);
		AppendExternalPersistentLevelForReload(Result.PackagesPendingHotReload, Result.PackagesPendingPurge);
	}
	return Result;
}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

void FSandboxInstance::EnumerateFileChanges(TFunctionRef<FProcessFileChangeSignature> InProcess, EFileEnumerationFlags InFlags) const
{
	FileSandboxCore::EnumerateFileChanges(*PlatformFile->GetLowerLevel(), GetRootDirectory(), ManifestContent, InProcess, InFlags);
}

TOptional<FDateTime> FSandboxInstance::GetSandboxedFileTimestamp(const FString& InFilePath) const
{
	return GetSandboxTimestamp(InFilePath, GetRootDirectory(), ManifestContent);
}

bool FSandboxInstance::DeletedPackageExistsInNonSandbox(const FString& InFilename) const
{
	return PlatformFile->DeletedPackageExistsInNonSandbox(InFilename);
}

void FSandboxInstance::OnFileDeletionStateChanged(const FSandboxedPlatformFilePath& InFile, ESandboxFileChange InOldState, ESandboxFileChange InNewState)
{
	const FNonSandboxPath NonSandboxPath = InFile.GetNonSandboxPath();

	const ESandboxRefreshFlags CurrentFlags = RefreshFlags.load();
	const bool bWasDeleted = InNewState == ESandboxFileChange::Removed;
	if (bWasDeleted)
	{
		ManifestContent.ModifiedFiles.Remove(NonSandboxPath);
		const bool bWasAdded = ManifestContent.AddedFiles.Remove(NonSandboxPath);
		if (!bWasAdded)
		{
			ManifestContent.DeletedFiles.Add(NonSandboxPath);
		}
		SaveManifest();
		RefreshFlags = CurrentFlags | ESandboxRefreshFlags::BroadcastChanges;
		return;
	}
	else if (InOldState == ESandboxFileChange::Removed
  		  && InNewState == ESandboxFileChange::Edited) 
	{
		ManifestContent.DeletedFiles.Remove(NonSandboxPath);
		ManifestContent.ModifiedFiles.Add(NonSandboxPath);
		SaveManifest();
		RefreshFlags = CurrentFlags | ESandboxRefreshFlags::BroadcastChanges;
		return;
	}
	
	RefreshFlags = CurrentFlags | ESandboxRefreshFlags::UpdateManifest | ESandboxRefreshFlags::BroadcastChanges;
}

bool FSandboxInstance::SaveManifest()
{
	TrimToReferencedMountPoints(ManifestContent);
	return FileSandboxCore::SaveManifest(ManifestContent, GetRootDirectory());
}

void FSandboxInstance::OnEndOfFrame()
{
	const ESandboxRefreshFlags CurrentFlags = RefreshFlags.exchange(ESandboxRefreshFlags::None);
	if (CurrentFlags == ESandboxRefreshFlags::None)
	{
		return;
	}
	
	// We update the manifest every time an update happens.
	// This is important as during the sandbox a crash could happen.
	// This way, when the sandbox is loaded again after a crash it's up to date. 
	if (EnumHasAnyFlags(CurrentFlags, ESandboxRefreshFlags::UpdateManifest))
	{
		FFileSandboxCore_ManifestData NewManifest;
		NewManifest.Initialize();
		EnumerateFileChanges([&NewManifest](const FSandboxedFileChangeInfo& Change)
		{
			Private::TrackFileChangeInManifest(NewManifest, Change);
			return EBreakBehavior::Continue;
		}, EFileEnumerationFlags::IncludeTimestamps);
		
		ManifestContent = MoveTemp(NewManifest);
		SaveManifest();
	}
	
	if (EnumHasAnyFlags(CurrentFlags, ESandboxRefreshFlags::BroadcastChanges))
	{
		OnSandboxedFilesChangedDelegate.Broadcast();
	}
}
}
