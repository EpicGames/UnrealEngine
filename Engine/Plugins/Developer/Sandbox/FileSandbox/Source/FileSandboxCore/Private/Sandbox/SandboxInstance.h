// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISandboxInstance.h"
#include "SandboxRefreshFlags.h"
#include "Data/ManifestData.h"
#include "Data/SandboxMetaData.h"
#include "Platform/SandboxPlatformFile.h"
#include "SourceControl/SandboxedSourceControl.h"
#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxCore
{
enum class ESandboxRefreshFlags : uint8;
struct FSandboxInitArgs;
	
class FSandboxInstance : public ISandboxInstance, public FNoncopyable
{
public:

	/** Initializes InRootDirectory with a new sandbox. */
	static TUniquePtr<FSandboxInstance> CreateNewSandbox(
		FString InRootDirectory, const FSandboxInitArgs& InInitArgs, const FFileSandboxCore_SandboxMetaData& InMetaData
		);
	/** Loads a preexisting sandbox from InRootDirectory. */
	static TUniquePtr<FSandboxInstance> LoadSandbox(FString InRootDirectory, const FSandboxInitArgs& InInitArgs);

	explicit FSandboxInstance(
		FString InRootDirectory,
		const FSandboxInitArgs& InitArgs,
		FFileSandboxCore_ManifestData InManifestData,
		FFileSandboxCore_SandboxMetaData InMetaData
		);
	virtual ~FSandboxInstance() override;

	using ISandboxInstance::RevertSandbox;

	const FFileSandboxCore_ManifestData& GetManifestContent() const { return ManifestContent; }
	IPlatformFile& GetLowerLevelPlatformFile() const { return *PlatformFile->GetLowerLevel(); }
	
	//~ Begin ISandboxInstance Interface
	virtual FPersistResult PersistSandbox(const FPersistArgs& InArgs) override;
	virtual FRevertResult RevertAll() override;
	virtual FRevertResult RevertSpecified(const TConstArrayView<FString>& InFiles) override;
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	virtual FRevertResult RevertSandbox(const FRevertArgs& InArgs) override;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	virtual void EnumerateFileChanges(TFunctionRef<FProcessFileChangeSignature> InProcess, EFileEnumerationFlags InFlags) const override;
	virtual TOptional<FDateTime> GetSandboxedFileTimestamp(const FString& InFilePath) const override;
	virtual bool DeletedPackageExistsInNonSandbox(const FString& InFilename) const override;
	virtual const FFileSandboxCore_SandboxMetaData& GetInitialMetaData() const override { return MetaData; }
	virtual const TCHAR* GetRootDirectory() const override { return *RootDirectory; }
	virtual FSimpleMulticastDelegate& OnSandboxedFilesChanged() override { return OnSandboxedFilesChangedDelegate; }
	//~ End ISandboxInstance Interface

private:

	/** Root directory in which we save all out content. */
	const FString RootDirectory;

	/** Current manifest data. */
	FFileSandboxCore_ManifestData ManifestContent;
	/**
	 * Metadata about this sandbox.
	 * 
	 * The sandbox does not actively use the metadata, but we keep it in memory in case anything happens to the file, e.g. the user manually deleting
	 * it while in the sandbox. This way we can always guarantee to the API user that there is always valid metadata.
	 */
	const FFileSandboxCore_SandboxMetaData MetaData;
	
	/** External customization for hot-reloading and purging of assets. If nothing was specified, this defaults to FDefaultPackageReloadHandler. */
	const TSharedRef<IPackageReloadHandler> PackageReloadHandler;

	/** Hooks into the engine's I/O system to redirect saving & loading of files. */
	TUniquePtr<FSandboxPlatformFile> PlatformFile;
	
	/** Sandboxes source control operations and reroutes them to use instead. */
	FSandboxedSourceControl SourceControlSandbox;
	
	/** Delegate invoked when the result of GatherChangedFiles has changed. */
	FSimpleMulticastDelegate OnSandboxedFilesChangedDelegate;
	
	/** Whether any sandboxed file changes have occurred this frame. */
	std::atomic<ESandboxRefreshFlags> RefreshFlags = ESandboxRefreshFlags::None;

	/** Marks the file as deleted in the manifest. */
	void OnFileDeletionStateChanged(const FSandboxedPlatformFilePath& InFile, ESandboxFileChange InOldState, ESandboxFileChange InNewState);

	/** Saves the manifest file. */
	bool SaveManifest();
	
	/** Processes pending work. */
	void OnEndOfFrame();
};
}

