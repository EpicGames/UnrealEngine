// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxManager.h"

#include "Data/VersionInfo.h"
#include "IFileSandboxCoreModule.h"
#include "ISandboxRepository.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "LogFileSandbox.h"
#include "Misc/CoreDelegates.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#include "Persist/DiffUtils.h"
#include "Platform/SanboxedFilePathUtils.h"
#include "Types/EBreakBehavior.h"
#include "Types/SandboxFileChange.h"
#include "Types/Manager/DeleteSandboxByDirectoryArgs.h"
#include "Types/Manager/DeleteSandboxResult.h"
#include "Types/Manager/LeaveSandboxResult.h"
#include "Types/Manager/LoadSandboxByDirectoryArgs.h"
#include "Types/Manager/LoadSandboxError.h"
#include "Types/Manager/NewSandboxArgs.h"
#include "Types/Manager/NewSandboxError.h"
#include "Types/Manager/SandboxCreationResult.h"
#include "Utils/SandboxDirectoryUtils.h"
#include "Utils/SandboxFileUtils.h"

namespace UE::FileSandboxCore
{
FSandboxManager::FSandboxManager()
{
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FSandboxManager::OnEngineExit);
}

FSandboxManager::~FSandboxManager()
{
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
}

FNewSandboxResult FSandboxManager::CreateNewSandbox(const FNewSandboxArgs& InArgs)
{
	const FString AbsoluteRoot = FPaths::ConvertRelativePathToFull(InArgs.SandboxBasePath);
	
	const FString& Name = InArgs.MetaData.Name;
	if (DetermineDirectorySuitability(Name, InArgs.SandboxBasePath) != EDirectorySuitability::Suitable)
	{
		return { MakeError(FNewSandboxError{ ENewSandboxErrorReason::UnsuitablePath }) };
	}

	if (!LeaveSandbox())
	{
		return { MakeError(FNewSandboxError{ ENewSandboxErrorReason::CannotLeaveSandbox }) };
	}
	
	const FString NewDirectory = FPaths::Combine(InArgs.SandboxBasePath, Name);
	TUniquePtr<FSandboxInstance> NewSandbox = FSandboxInstance::CreateNewSandbox(NewDirectory, InArgs.InitArgs, InArgs.MetaData);
	ISandboxInstance* SandboxRaw = NewSandbox.Get();
	if (!SandboxRaw)
	{
		return { MakeError(FNewSandboxError{ ENewSandboxErrorReason::IOError }) };
	}
	
	ActiveSandboxData.Emplace(MoveTemp(NewSandbox), InArgs.InitArgs.Lock);
	OnPostSandboxStartupDelegate.Broadcast(*SandboxRaw);
	return { MakeValue(SandboxRaw) };
}

FLoadSandboxResult FSandboxManager::LoadSandbox(const FLoadSandboxByDirectoryArgs& InLoadArgs)
{
	const FString AbsoluteRoot = FPaths::ConvertRelativePathToFull(InLoadArgs.SandboxRootPath);
	if (!IsRootSandboxDirectory(AbsoluteRoot))
	{
		UE_LOGF(LogFileSandbox, Error, "Directory %ls does not contain any sandbox", *AbsoluteRoot);
		return { MakeError(FLoadSandboxError{ ELoadSandboxLoadErrorReason::InvalidDirectory }) };
	}

	// This could happen if the sandbox is placed in the root e.g. "D:/".
	// It means that the sandbox was not created in GetBaseSandboxDirectory.
	// Names are just a simplified concept for managing sandboxes that are placed in this default folder.
	// This is a custom sandbox so it's beyond the default workflow.
	const FString LastDirectoryName = FPaths::GetPathLeaf(AbsoluteRoot);
	UE_CLOGF(LastDirectoryName.IsEmpty(), LogFileSandbox, Warning, "Sandbox placed %ls will not have any name.", *AbsoluteRoot);
	const TOptional<FString> SandboxName = LastDirectoryName.IsEmpty() ? LastDirectoryName : TOptional<FString>{};

	// Check version compatibility BEFORE leaving current sandbox (UE-356522)
	// If the version is incompatible, we want to fail early without disrupting the user's current state
	const FFileSandboxCore_VersionInfo SandboxVersionInfo = LoadVersionInfo(AbsoluteRoot);
	if (SandboxVersionInfo.IsInitialized())
	{
		FFileSandboxCore_EngineVersionInfo CurrentVersion;
		CurrentVersion.Initialize(FEngineVersion::Current());

		if (!SandboxVersionInfo.EngineVersion.IsCompatibleWith(CurrentVersion))
		{
			UE_LOGF(LogFileSandbox, Error, "Cannot load sandbox at %ls: Sandbox was created with version %ls but current version is %ls",
				*AbsoluteRoot,
				*SandboxVersionInfo.EngineVersion.ToString(),
				*CurrentVersion.ToString());
			return { MakeError(FLoadSandboxError{ ELoadSandboxLoadErrorReason::IncompatibleVersion }) };
		}
	}

	if (!LeaveSandbox())
	{
		return { MakeError(FLoadSandboxError{ ELoadSandboxLoadErrorReason::CannotLeaveSandbox }) };
	}

	TUniquePtr<FSandboxInstance> NewSandbox = FSandboxInstance::LoadSandbox(AbsoluteRoot, InLoadArgs.InitArgs);
	ISandboxInstance* SandboxRaw = NewSandbox.Get();
	if (!SandboxRaw)
	{
		return { MakeError(FLoadSandboxError{ ELoadSandboxLoadErrorReason::IOError }) };
	}
	
	ActiveSandboxData.Emplace(MoveTemp(NewSandbox), InLoadArgs.InitArgs.Lock);
	OnPostSandboxStartupDelegate.Broadcast(*SandboxRaw);
	return { MakeValue(SandboxRaw) };
}

FDeleteSandboxResult FSandboxManager::DeleteSandbox(const FDeleteSandboxByDirectoryArgs& InArgs)
{
	if (!IsRootSandboxDirectory(InArgs.Directory))
	{
		return EDeleteSandboxErrorCode::InvalidDirectory;
	}
	
	ISandboxInstance* ActiveSandbox = GetActiveSandboxInstance();
	const bool bNeedsLeave = ActiveSandbox && FPaths::IsSamePath(ActiveSandbox->GetRootDirectory(), InArgs.Directory);
	if (bNeedsLeave && !LeaveSandbox())
	{
		return EDeleteSandboxErrorCode::CannotLeaveSandbox;
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	return PlatformFile.DeleteDirectoryRecursively(*InArgs.Directory)
		? EDeleteSandboxErrorCode::Success
		: EDeleteSandboxErrorCode::IOError;
}

FLeaveSandboxResult FSandboxManager::LeaveSandbox(const FLeaveSandboxArgs& InLeaveArgs)
{
	if (!ActiveSandboxData)
	{
		return ELeaveSandboxErrorCode::Success;
	}
	
	if (const ELeaveSandboxErrorCode ErrorCode = CanLeaveSandboxWithReason()
		; ErrorCode != ELeaveSandboxErrorCode::Success)
	{
		return ErrorCode;
	}
	
	OnPreSandboxShutdownDelegate.Broadcast(*ActiveSandboxData->Sandbox.Get());
	ActiveSandboxData.Reset();
	OnPostSandboxShutdownDelegate.Broadcast();
	return ELeaveSandboxErrorCode::Success;
}

void FSandboxManager::EnumerateFileChanges(const FString& InSandboxRootPath, TFunctionRef<FProcessFileChangeSignature> InProcess, EFileEnumerationFlags InFlags) const
{
	FSandboxInstance* Instance = GetSandboxInstanceInternal();
	if (Instance && FPaths::IsSamePath(Instance->GetRootDirectory(), InSandboxRootPath))
	{
		FileSandboxCore::EnumerateFileChanges(
			Instance->GetLowerLevelPlatformFile(), Instance->GetRootDirectory(), Instance->GetManifestContent(), InProcess, InFlags
			);
		return;
	}
	
	if (const TOptional<FFileSandboxCore_ManifestData> Manifest = LoadManifest(InSandboxRootPath))
	{
		IPlatformFile& PlatformFile = Instance 
			? *FPlatformFileManager::Get().GetPlatformFile().GetLowerLevel() 
			: FPlatformFileManager::Get().GetPlatformFile();
		FileSandboxCore::EnumerateFileChanges(PlatformFile, InSandboxRootPath, *Manifest, InProcess, InFlags);
	}
}

TOptional<FDateTime> FSandboxManager::GetSandboxedFileTimestamp(const FString& InSandboxRootPath, const FString& InFilePath) const
{
	FSandboxInstance* Instance = GetSandboxInstanceInternal();
	if (Instance && FPaths::IsSamePath(Instance->GetRootDirectory(), InSandboxRootPath))
	{
		return GetSandboxTimestamp(InFilePath, InSandboxRootPath, Instance->GetManifestContent());
	}
	
	const TOptional<FFileSandboxCore_ManifestData> Manifest = LoadManifest(InSandboxRootPath);
	return Manifest ? GetSandboxTimestamp(InFilePath, InSandboxRootPath, *Manifest) : TOptional<FDateTime>();
}

void FSandboxManager::OnEngineExit()
{
	LeaveSandbox();
}
}
