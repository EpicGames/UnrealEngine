// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISandboxManager.h"
#include "SandboxInstance.h"
#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxCore
{
/** Manges the sandbox lifetime for the module */
class FSandboxManager : public ISandboxManager, public FNoncopyable
{
public:

	FSandboxManager();
	virtual ~FSandboxManager() override;

	//~ Begin ISandboxManager Interface
	virtual FNewSandboxResult CreateNewSandbox(const FNewSandboxArgs& InArgs) override;
	virtual FLoadSandboxResult LoadSandbox(const FLoadSandboxByDirectoryArgs& InLoadArgs) override;
	virtual FDeleteSandboxResult DeleteSandbox(const FDeleteSandboxByDirectoryArgs& InArgs) override;
	virtual FLeaveSandboxResult LeaveSandbox(const FLeaveSandboxArgs& InLeaveArgs = FLeaveSandboxArgs()) override;
	virtual ISandboxLock* GetActiveLock() const override { return ActiveSandboxData ? ActiveSandboxData->Lock.Get() : nullptr; }
	virtual ISandboxInstance* GetActiveSandboxInstance() const override { return ActiveSandboxData ? ActiveSandboxData->Sandbox.Get() : nullptr; }
	virtual void EnumerateFileChanges(const FString& InSandboxRootPath, TFunctionRef<FProcessFileChangeSignature> InProcess, EFileEnumerationFlags InFlags) const override;
	virtual TOptional<FDateTime> GetSandboxedFileTimestamp(const FString& InSandboxRootPath, const FString& InFilePath) const override;
	virtual FSandboxInstanceEvent& OnPostSandboxStartup() override { return OnPostSandboxStartupDelegate; }
	virtual FSandboxInstanceEvent& OnPreSandboxShutdown() override { return OnPreSandboxShutdownDelegate; }
	virtual FSandboxShutdownEvent& OnPostSandboxShutdown() override { return OnPostSandboxShutdownDelegate; }
	//~ End ISandboxManager Interface

private:
	
	struct FActiveSandboxData
	{
		/** Current sandbox. Nullptr if not in any sandbox. */
		TUniquePtr<FSandboxInstance> Sandbox;
		
		/** Optional. If set, determines whether it is allowed to leave the sandbox. */
		TSharedPtr<ISandboxLock> Lock;

		explicit FActiveSandboxData(TUniquePtr<FSandboxInstance> Sandbox, TSharedPtr<ISandboxLock> InLock)
			: Sandbox(MoveTemp(Sandbox))
			, Lock(InLock)
		{}
	};
	/** Set while in a sandbox */
	TOptional<FActiveSandboxData> ActiveSandboxData;

	/** Invoked after a sandbox has been successfully created. */
	FSandboxInstanceEvent OnPostSandboxStartupDelegate;
	/** Invoked when it has been decided, that a sandbox instance will be taken down, but before any core destruction logic runs. */
	FSandboxInstanceEvent OnPreSandboxShutdownDelegate;
	/** Invoked at the end of shutting down a sandbox, i.e. after all core destruction logic has run. */
	FSandboxShutdownEvent OnPostSandboxShutdownDelegate;

	/** Stops the current sandbox. */
	void OnEngineExit();
	
	FSandboxInstance* GetSandboxInstanceInternal() const { return ActiveSandboxData ? ActiveSandboxData->Sandbox.Get() : nullptr;  }
};
}


