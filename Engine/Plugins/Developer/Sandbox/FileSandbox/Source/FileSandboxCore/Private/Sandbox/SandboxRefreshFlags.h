// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::FileSandboxCore
{
/** Flags for refreshing the sandbox instance. */
enum class ESandboxRefreshFlags : uint8
{
	None,
	
	/** Broadcast that the set of sandboxed file changes has changed. */
	BroadcastChanges = 1 << 0,
	/** The manifest file needs to be resaved. */
	UpdateManifest = 1 << 1
};
ENUM_CLASS_FLAGS(ESandboxRefreshFlags);
}