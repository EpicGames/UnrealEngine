// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxSourceControlProviderProxy.h"

#include "Features/IModularFeatures.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "SandboxSourceControlStateProxy.h"
#include "SourceControlOperations.h"

namespace UE::FileSandboxCore
{
namespace Private
{
static bool IsProhibitedOp(const FSourceControlOperationRef& InOperation)
{
	const FName OpName = InOperation->GetName();
	// We're not supposed to alter source control state while in sandbox - we touch the files directly and only affect source control when persisting.
	// Some editor operations issue a revert followed by file op, e.g. deleting an edited asset will first revert the asset, then delete it.
	return OpName == TEXT("Revert")	
		// This would show a popup a status update widget, don't show it in sandbox.
		|| OpName == TEXT("UpdateStatus"); 
}
	
static TOptional<ECommandResult::Type> HandleSandboxedOp(const FSourceControlOperationRef& InOperation, const TArray<FString>& InFiles)
{
	const FName OpName = InOperation->GetName();
	
	// Edits should not show up in the external source control programming until they are persisted by the sandbox
	if (OpName == TEXT("CheckOut"))
	{
		return ECommandResult::Succeeded;
	}
	
	// Added files should not show up in the external source control programming until they are persisted by the sandbox
	if (OpName == TEXT("MarkForAdd"))
	{
		return ECommandResult::Succeeded;
	}
	
	// The contract ISourceControlProvider handles deleting the file. We'll just reroute it to our sandbox. 
	// If the changes end up being persisted, the source control changes will be routed to the real source control then.
	if (OpName == TEXT("Delete"))
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		bool bAllSuccessful = true;
		for (const FString& InFile : InFiles)
		{
			bAllSuccessful &= PlatformFile.DeleteFile(*InFile);
		}
		return bAllSuccessful ? ECommandResult::Succeeded : ECommandResult::Failed;
	}
	
	if (OpName == TEXT("Copy"))
	{
		if (ensure(InFiles.Num() == 1))
		{
			TSharedRef<FCopy> CopyOp = StaticCastSharedRef<FCopy>(InOperation);
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			const bool bSuccess = PlatformFile.MoveFile(*CopyOp->GetDestination(), *InFiles[0]);
			return bSuccess ? ECommandResult::Succeeded : ECommandResult::Failed;
		}
		return ECommandResult::Failed;
	}
	
	return {};
}
	
static TOptional<ECommandResult::Type> InterceptSourceControlOp(
	const FSourceControlOperationRef& InOperation , const TArray<FString>& InFiles
	)
{
	// Sandbox becomes temporarily disabled when we're persisting changes.
	if (!IFileManager::Get().IsSandboxEnabled())
	{
		return {};
	}
	
	// Handle forwarding equivalents of edit, add, and remove to sandbox.
	if (const TOptional<ECommandResult::Type> Result = HandleSandboxedOp(InOperation, InFiles))
	{
		return *Result;
	}
	
	if (IsProhibitedOp(InOperation))
	{
		return ECommandResult::Succeeded;
	}
	
	return {};
}
}
	
FSandboxSourceControlProviderProxy::FSandboxSourceControlProviderProxy(
	FSandboxInstance& InInstance, const FSourceControlProxyConfig& InStateProxyConfig
	)
	: Super(&ISourceControlModule::Get().GetProvider())
	, Instance(InInstance)
	, StateProxyConfig(InStateProxyConfig)
{
	IModularFeatures::Get().RegisterModularFeature("SourceControl", this);
}

FSandboxSourceControlProviderProxy::~FSandboxSourceControlProviderProxy()
{
	IModularFeatures::Get().UnregisterModularFeature("SourceControl", this);
}

const FName& FSandboxSourceControlProviderProxy::GetName() const
{
	static FName Name(TEXT("File Sandbox"));
	return Name;
}

ECommandResult::Type FSandboxSourceControlProviderProxy::Execute(
	const FSourceControlOperationRef& InOperation,
	FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency,
	const FSourceControlOperationComplete& InOperationCompleteDelegate
	)
{
	if (const TOptional<ECommandResult::Type> Result = Private::InterceptSourceControlOp(InOperation, InFiles))
	{
		InOperationCompleteDelegate.ExecuteIfBound(InOperation, *Result);
		return *Result;
	}
	
	return Super::Execute(InOperation, InChangelist, InFiles, InConcurrency, InOperationCompleteDelegate);
}

ECommandResult::Type FSandboxSourceControlProviderProxy::GetState(
	const TArray<FString>& InFiles, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage
	)
{
	ECommandResult::Type Result = ECommandResult::Failed;

	if (UnderlyingProvider)
	{
		Result = UnderlyingProvider->GetState(InFiles, OutState, InStateCacheUsage);
	}

	if (Result == ECommandResult::Failed)
	{
		// Even when the provider fails, return a dummy proxy so we override some of the behavior, such as the IsLocal override for redirectors.
		Result = ECommandResult::Succeeded;

		OutState.Reset();
		for (const FString& File : InFiles)
		{
			OutState.Add(MakeShared<FSandboxSourceControlStateProxy, ESPMode::ThreadSafe>(File, StateProxyConfig));
		}
	}
	else
	{
		for (FSourceControlStateRef& State : OutState)
		{
			State = MakeShared<FSandboxSourceControlStateProxy, ESPMode::ThreadSafe>(State, StateProxyConfig);
		}
	}

	return Result;
}
}
