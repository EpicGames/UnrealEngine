// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "SandboxSourceControlStateProxy.h"
#include "SourceControlProviderProxyBase.h"
#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxCore
{
class FSandboxInstance;
struct FSourceControlProxyConfig;

/** 
 * Intercepts any file changes the engine attempts to make and routes them FSandboxInstance.
 * 
 * When set-up with source control, some tools in the engine route I/O operations to the external source control tool instead of to IPlatformFile.
 * The point of this is that the external program handles the I/O operations, e.g. "p4 delete" deletes the file and registers it with a changelist.
 * 
 * However, during sandbox we're not supposed to make any external changes - so we'll "swallow" those changes and re-reroute to FSandboxInstance instead.
 * FSandboxInstance will handle re-issuing the source control commands again if the changes are persisted.
 */
class FSandboxSourceControlProviderProxy 
	: public FBaseSourceControlProviderProxy
	, public FNoncopyable
{
	using Super = FBaseSourceControlProviderProxy;
public:
	
	explicit FSandboxSourceControlProviderProxy(FSandboxInstance& InInstance UE_LIFETIMEBOUND, const FSourceControlProxyConfig& InStateProxyConfig);
	~FSandboxSourceControlProviderProxy();
	
	void SetUnderlyingSourceControlProvider(ISourceControlProvider* InProvider) { UnderlyingProvider = InProvider; }
	
	//~ Begin ISourceControlProvider Interface
	virtual const FName& GetName() const override;
	virtual ECommandResult::Type Execute(
		const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, 
		EConcurrency::Type InConcurrency = EConcurrency::Synchronous, 
		const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() 
		) override;
	virtual ECommandResult::Type GetState(
		const TArray<FString>& InFiles, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage
		) override;
	//~ End ISourceControlProvider Interface
	
private:
	
	/** Instance of the sandbox for which we've been created. */
	FSandboxInstance& Instance;
	
	/** Config to construct the proxy state instances with. */
	const FSourceControlProxyConfig StateProxyConfig;
};
}


