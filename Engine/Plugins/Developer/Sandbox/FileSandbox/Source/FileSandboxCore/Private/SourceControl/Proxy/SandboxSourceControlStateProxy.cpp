// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxSourceControlStateProxy.h"

#include "HAL/IConsoleManager.h"

namespace UE::FileSandboxCore
{
enum EIsLocalOverrideMode
{
	NoOverride,
	ForceLocal,
	ForceNonLocal
};
static TAutoConsoleVariable<int32> CVarOverrideIsLocal(
	TEXT("Sandbox.IsLocalOverrideMode"), 
	EIsLocalOverrideMode::NoOverride, 
	TEXT("Debug util for ISourceControlState::IsLocal while in sandbox.\n0 - No override (return what SC says),\n1 - Force IsLocal() == true,\n2 - Force IsLocal() == false")
	);
	
bool FSandboxSourceControlStateProxy::IsLocal() const
{
	switch (CVarOverrideIsLocal.GetValueOnAnyThread())
	{
	case EIsLocalOverrideMode::ForceLocal: return true;
	case EIsLocalOverrideMode::ForceNonLocal: return false;
	case EIsLocalOverrideMode::NoOverride: [[fallthrough]];
	default: return Config.IsLocalOverride ? *Config.IsLocalOverride : Super::IsLocal();
	}
}
}
