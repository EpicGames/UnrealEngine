// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SourceControlProxyConfig.h"
#include "SourceControlStateProxyBase.h"

namespace UE::FileSandboxCore
{
/** 
 * The purpose of this proxy state is to force files to optionally be treated as non-local. 
 * This achieves that when files are renamed, redirectors are created as well. Some features, such as Multi-User / Concert, require redirectors.
 */
class FSandboxSourceControlStateProxy : public FSourceControlStateProxyBase
{
	using Super = FSourceControlStateProxyBase;
public:
	
	explicit FSandboxSourceControlStateProxy(FSourceControlStatePtr InProxyState, const FSourceControlProxyConfig& InConfig) 
		: FSourceControlStateProxyBase(MoveTemp(InProxyState)) 
		, Config(InConfig)
	{}
	explicit FSandboxSourceControlStateProxy(FString InFilename, const FSourceControlProxyConfig& InConfig) 
		: FSourceControlStateProxyBase(MoveTemp(InFilename)) 
		, Config(InConfig)
		{}

	//~ Begin ISourceControlState Interface
	virtual bool IsLocal() const override;
	//~ End ISourceControlState Interface
	
private:
	
	/** Config for our behaviour should override the proxied state. */
	const FSourceControlProxyConfig Config;
};
}
