// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#if SOURCE_CONTROL_WITH_SLATE
#include "Widgets/SNullWidget.h"
#endif

#define WRAP(FuncName, ...) if (ISourceControlProvider* Provider = GetUnderlyingProvider()) { Provider->FuncName(__VA_ARGS__); }
#define WRAP_RET(Default, FuncName, ...) if (ISourceControlProvider* Provider = GetUnderlyingProvider()) { return Provider->FuncName(__VA_ARGS__); } return Default; 

namespace UE::FileSandboxCore
{
/** 
 * Forwards all calls to an optional, proxied source control provider. 
 * Acts as base class to reduce boilerplate when proxying source control classes.
 */
class FBaseSourceControlProviderProxy : public ISourceControlProvider
{
public:
	
	explicit FBaseSourceControlProviderProxy(ISourceControlProvider* InActualProvider)
		: UnderlyingProvider(InActualProvider)
	{}
	
	//~ Begin ISourceControlProvider Interface
	virtual void Init(bool bForceConnection = true) override
	{
		WRAP(Init, bForceConnection)
	}
	virtual void Close() override
	{
		WRAP(Close);
	}
	virtual FText GetStatusText() const override
	{
		WRAP_RET(FText::GetEmpty(), GetStatusText)
	}
	virtual TMap<EStatus, FString> GetStatus() const override
	{
		WRAP_RET({}, GetStatus)
	}
	virtual bool IsEnabled() const override
	{
		WRAP_RET(true, IsEnabled)
	}
	virtual bool IsAvailable() const override
	{
		WRAP_RET(true, IsAvailable)
	}
	virtual bool QueryStateBranchConfig(const FString& ConfigSrc, const FString& ConfigDest) override
	{
		WRAP_RET(false, QueryStateBranchConfig, ConfigSrc, ConfigDest)
	}
	virtual void RegisterStateBranches(const TArray<FString>& BranchNames, const FString& ContentRoot) override
	{
		WRAP(RegisterStateBranches, BranchNames, ContentRoot)
	}
	virtual int32 GetStateBranchIndex(const FString& BranchName) const override
	{
		WRAP_RET(INDEX_NONE, GetStateBranchIndex, BranchName);
	}
	virtual bool GetStateBranchAtIndex(int32 BranchIndex, FString& OutBranchName) const override
	{
		WRAP_RET(false, GetStateBranchAtIndex, BranchIndex, OutBranchName);
	}
	virtual ECommandResult::Type GetState(const TArray<FString>& InFiles, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage) override
	{
		WRAP_RET(ECommandResult::Failed, GetState, InFiles, OutState, InStateCacheUsage)
	}
	virtual ECommandResult::Type GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage) override
	{
		WRAP_RET(ECommandResult::Failed, GetState, InChangelists, OutState, InStateCacheUsage)
	}
	virtual TArray<FSourceControlStateRef> GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const override
	{
		WRAP_RET({}, GetCachedStateByPredicate, Predicate)
	}
	virtual FDelegateHandle RegisterSourceControlStateChanged_Handle(const FSourceControlStateChanged::FDelegate& SourceControlStateChanged) override
	{
		WRAP_RET({}, RegisterSourceControlStateChanged_Handle, SourceControlStateChanged)
	}
	virtual void UnregisterSourceControlStateChanged_Handle(FDelegateHandle Handle) override
	{
		WRAP(UnregisterSourceControlStateChanged_Handle, Handle)
	}
	virtual ECommandResult::Type Execute(const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete()) override
	{
		WRAP_RET(ECommandResult::Failed, Execute, InOperation, InChangelist, InFiles, InConcurrency, InOperationCompleteDelegate)
	} 
	virtual bool CanExecuteOperation(const FSourceControlOperationRef& InOperation) const override
	{
		WRAP_RET(false, CanExecuteOperation, InOperation)
	}
	virtual bool CanCancelOperation(const FSourceControlOperationRef& InOperation) const override
	{
		WRAP_RET(false, CanCancelOperation, InOperation)
	}
	virtual void CancelOperation(const FSourceControlOperationRef& InOperation) override
	{
		WRAP(CancelOperation, InOperation)
	}
	virtual TArray<TSharedRef<class ISourceControlLabel>> GetLabels(const FString& InMatchingSpec) const override
	{
		WRAP_RET({}, GetLabels, InMatchingSpec)
	}
	virtual TArray<FSourceControlChangelistRef> GetChangelists(EStateCacheUsage::Type InStateCacheUsage) override
	{
		WRAP_RET({}, GetChangelists, InStateCacheUsage)
	}
	virtual bool UsesLocalReadOnlyState() const override
	{
		WRAP_RET(false, UsesLocalReadOnlyState)
	}
	virtual bool UsesChangelists() const override
	{
		WRAP_RET(false, UsesChangelists)
	}
	virtual bool UsesUncontrolledChangelists() const override
	{
		WRAP_RET(false, UsesUncontrolledChangelists)
	}
	virtual bool UsesCheckout() const override
	{
		WRAP_RET(false, UsesCheckout)
	}
	virtual bool UsesFileRevisions() const override
	{
		WRAP_RET(false, UsesFileRevisions)
	}
	virtual bool UsesSnapshots() const override
	{
		WRAP_RET(false, UsesSnapshots)
	}
	virtual bool UsesSoftRevertOnDelete() const override
	{
		WRAP_RET(false, UsesSoftRevertOnDelete)
	}
	virtual bool AllowsDiffAgainstDepot() const override
	{
		WRAP_RET(false, AllowsDiffAgainstDepot)
	}
	virtual TOptional<bool> HasChangesToSync() const override
	{
		WRAP_RET({}, HasChangesToSync)
	}
	virtual TOptional<bool> HasChangesToCheckIn() const override
	{
		WRAP_RET({}, HasChangesToCheckIn)
	}
	virtual void Tick() override
	{
		WRAP(Tick)
	}
#if SOURCE_CONTROL_WITH_SLATE
	virtual TSharedRef<class SWidget> MakeSettingsWidget() const override
	{
		WRAP_RET(SNullWidget::NullWidget, MakeSettingsWidget)
	}
#endif
	//~ End ISourceControlProvider Interface
	
protected:
	
	ISourceControlProvider* UnderlyingProvider;
	
	ISourceControlProvider* GetUnderlyingProvider() const 
	{
		return UnderlyingProvider;
	}
};
}

#undef WRAP
#undef WRAP_RET