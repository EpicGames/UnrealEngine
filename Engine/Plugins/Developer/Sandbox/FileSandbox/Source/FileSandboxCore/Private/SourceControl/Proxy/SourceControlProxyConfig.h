// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Types/Manager/SandboxInitArgs.h"

namespace UE::FileSandboxCore
{
struct FSourceControlProxyConfig
{
	/** If set, return this value from IsLocal. */
	TOptional<bool> IsLocalOverride;
};
	
/** @return Source control proxy config constructed from the sandbox init args. */
FSourceControlProxyConfig MakeConfigFromInitArgs(const FSandboxInitArgs& InInitArgs);
}

namespace UE::FileSandboxCore
{
inline FSourceControlProxyConfig MakeConfigFromInitArgs(const FSandboxInitArgs& InInitArgs)
{
	const bool bForceNonLocal = EnumHasAnyFlags(InInitArgs.Flags, ESandboxInitFlags::ForceRedirectorsOnRenameInSourceControl);
	return FSourceControlProxyConfig
	{ 
		.IsLocalOverride = bForceNonLocal ? false : TOptional<bool>{}
	};
}
}
