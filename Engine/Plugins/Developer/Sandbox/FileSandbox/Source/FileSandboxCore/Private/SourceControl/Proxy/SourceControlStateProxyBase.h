// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceControlState.h"
#include "HAL/FileManager.h"

namespace UE::FileSandboxCore
{
/** 
 * Forwards all calls to an optional, proxied source control state. 
 * Acts as base class to reduce boilerplate when proxying source control classes.
 */
class FSourceControlStateProxyBase : public ISourceControlState
{
public:
	
	explicit FSourceControlStateProxyBase(FSourceControlStatePtr InProxyState) : ActualState(MoveTemp(InProxyState)) {}
	explicit FSourceControlStateProxyBase(FString InFilename) 
		: ActualState(nullptr)
		, CachedFilename(MoveTemp(InFilename))
		, CachedTimestamp(IFileManager::Get().GetTimeStamp(*CachedFilename))
	{}
	
	//~ Begin ISourceControlState Interface
	virtual int32 GetHistorySize() const override { return ActualState.IsValid() ? ActualState->GetHistorySize() : 0; }
	virtual TSharedPtr<ISourceControlRevision> GetHistoryItem(int32 HistoryIndex) const override 
	{ 
		return ActualState.IsValid() ? ActualState->GetHistoryItem(HistoryIndex) : TSharedPtr<ISourceControlRevision>();
	}
	virtual TSharedPtr<ISourceControlRevision> FindHistoryRevision(int32 RevisionNumber) const override
	{
		return ActualState.IsValid() ? ActualState->FindHistoryRevision(RevisionNumber) : TSharedPtr<ISourceControlRevision>();
	}
	virtual TSharedPtr<ISourceControlRevision> FindHistoryRevision(const FString& InRevision) const override
	{
		return ActualState.IsValid() ? ActualState->FindHistoryRevision(InRevision) : TSharedPtr<ISourceControlRevision>();
	}
	virtual TSharedPtr<ISourceControlRevision> GetCurrentRevision() const override
	{
		return ActualState.IsValid() ? ActualState->GetCurrentRevision() : TSharedPtr<ISourceControlRevision>();
	}
	virtual FResolveInfo GetResolveInfo() const override 
	{ 
		return ActualState.IsValid() ? ActualState->GetResolveInfo() : FResolveInfo();
	}
	virtual FSlateIcon GetIcon() const override
	{
		return ActualState.IsValid() ? ActualState->GetIcon() : FSlateIcon();
	}
	virtual FText GetDisplayName() const override
	{
		return ActualState.IsValid() ? ActualState->GetDisplayName() : FText::GetEmpty();
	}
	virtual FText GetDisplayTooltip() const override
	{
		return ActualState.IsValid() ? ActualState->GetDisplayTooltip() : FText::GetEmpty();
	}
	virtual const FString& GetFilename() const override
	{
		return ActualState.IsValid() ? ActualState->GetFilename() : CachedFilename;
	}
	virtual const FDateTime& GetTimeStamp() const override
	{
		return ActualState.IsValid() ? ActualState->GetTimeStamp() : CachedTimestamp;
	}
	virtual bool CanCheckIn() const override
	{
		return ActualState.IsValid() && ActualState->CanCheckIn();
	}
	virtual bool CanCheckout() const override
	{
		return ActualState.IsValid() && ActualState->CanCheckout();
	}
	virtual bool IsCheckedOut() const override
	{
		return ActualState.IsValid() && ActualState->IsCheckedOut();
	}
	virtual bool IsCheckedOutOther(FString* Who = 0) const override
	{
		return ActualState.IsValid() && ActualState->IsCheckedOutOther(Who);
	}
	virtual bool IsCheckedOutInOtherBranch(const FString& CurrentBranch = FString()) const override
	{
		return ActualState.IsValid() && ActualState->IsCheckedOutInOtherBranch(CurrentBranch);
	}
	virtual bool IsModifiedInOtherBranch(const FString& CurrentBranch = FString()) const override
	{
		return ActualState.IsValid() && ActualState->IsModifiedInOtherBranch(CurrentBranch);
	}
	virtual bool IsCheckedOutOrModifiedInOtherBranch(const FString& CurrentBranch = FString()) const override
	{
		return ActualState.IsValid() && ActualState->IsCheckedOutOrModifiedInOtherBranch(CurrentBranch);
	}
	virtual TArray<FString> GetCheckedOutBranches() const override
	{
		return ActualState.IsValid() ? ActualState->GetCheckedOutBranches() : TArray<FString>();
	}
	virtual FString GetOtherUserBranchCheckedOuts() const override
	{
		return ActualState.IsValid() ? ActualState->GetOtherUserBranchCheckedOuts() : FString();
	}
	virtual bool GetOtherBranchHeadModification(FString& HeadBranchOut, FString& ActionOut, int32& HeadChangeListOut) const override
	{
		return ActualState.IsValid() && ActualState->GetOtherBranchHeadModification(HeadBranchOut, ActionOut, HeadChangeListOut);
	}
	virtual bool IsCurrent() const override
	{
		return ActualState.IsValid() && ActualState->IsCurrent();
	}
	virtual bool IsSourceControlled() const override
	{
		return ActualState.IsValid() && ActualState->IsSourceControlled();
	}
	virtual bool IsLocal() const override
	{
		return ActualState.IsValid() && ActualState->IsLocal();
	}
	virtual bool IsAdded() const override
	{
		return ActualState.IsValid() && ActualState->IsAdded();
	}
	virtual bool IsDeleted() const override
	{
		return ActualState.IsValid() && ActualState->IsDeleted();
	}
	virtual bool IsIgnored() const override
	{
		return ActualState.IsValid() && ActualState->IsIgnored();
	}
	virtual bool CanEdit() const override
	{
		return ActualState.IsValid() && ActualState->CanEdit();
	}
	virtual bool CanDelete() const override
	{
		return ActualState.IsValid() && ActualState->CanDelete();
	}
	virtual bool IsUnknown() const override
	{
		return ActualState.IsValid() && ActualState->IsUnknown();
	}
	virtual bool IsModified() const override
	{
		return ActualState.IsValid() && ActualState->IsModified();
	}
	virtual bool CanAdd() const override
	{
		return ActualState.IsValid() && ActualState->CanAdd();
	}
	virtual bool IsConflicted() const override
	{
		return ActualState.IsValid() && ActualState->IsConflicted();
	}
	virtual bool CanRevert() const override
	{
		return ActualState.IsValid() && ActualState->CanRevert();
	}
	//~ End ISourceControlState Interface
	
protected:
	
	/** The underlying state we proxy through. */
	FSourceControlStatePtr ActualState;
	
	/** The name of the file we represent (only when ActualState is null) */
	FString CachedFilename;
	/** The timestamp of the file we represent (only when ActualState is null) */
	FDateTime CachedTimestamp;
};
}
