// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxedSourceControl.h"

namespace UE::FileSandboxCore
{
FSandboxedSourceControl::FSandboxedSourceControl(FSandboxInstance& InSandboxInstance, const FSourceControlProxyConfig& InStateProxyConfig)
	: SandboxInstance(InSandboxInstance)
	, SourceControlProviderProxy(InSandboxInstance, InStateProxyConfig)
	, Override(SourceControlProviderProxy)
{
	Override.OnProxiedProviderChanged().AddLambda([this](ISourceControlProvider& InRestoreProvider)
	{
		SourceControlProviderProxy.SetUnderlyingSourceControlProvider(&InRestoreProvider);
	});
}
}
