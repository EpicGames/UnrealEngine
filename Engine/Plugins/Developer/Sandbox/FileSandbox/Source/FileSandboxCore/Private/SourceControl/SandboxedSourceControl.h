// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Proxy/SandboxSourceControlProviderProxy.h"
#include "ScopedSourceControlOverride.h"
#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxCore
{
class FSandboxInstance;
struct FSourceControlProxyConfig;
	
/** Interface for FSandboxInstance to interact with source control. */
class FSandboxedSourceControl : public FNoncopyable
{
public:

	explicit FSandboxedSourceControl(FSandboxInstance& InSandboxInstance UE_LIFETIMEBOUND, const FSourceControlProxyConfig& InStateProxyConfig);
	
	/** @return The real source control provider that we're overriding. */
	ISourceControlProvider& GetProxiedSourceControl() const { return Override.GetProxiedSourceControlProvider(); }
	
private:
	
	/** Owning sandbox instance with which source control interacts. */
	FSandboxInstance& SandboxInstance;
	
	/** The sandbox proxy that should be active for as long as the sandbox exists. */
	FSandboxSourceControlProviderProxy SourceControlProviderProxy;
	
	/** Sets the global source control and handles it externally changing. */
	FScopedSourceControlOverride Override;
};
}

