// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScopedSourceControlOverride.h"

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Features/IModularFeatures.h"

namespace UE::FileSandboxCore
{
FScopedSourceControlOverride::FScopedSourceControlOverride(ISourceControlProvider& InInitialProvider)
	: DesiredRootProvider(&InInitialProvider)
	, ProxiedSourceControlProvider(&ISourceControlModule::Get().GetProvider())
	, ProviderChangedHandle([this]
	{
		ISourceControlModule& SourceControl = ISourceControlModule::Get();
		return SourceControl.RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateRaw(this, &FScopedSourceControlOverride::HandleProviderChanged));
	}())
{
	SetRootProvider(*DesiredRootProvider);
}

FScopedSourceControlOverride::~FScopedSourceControlOverride()
{
	if (ISourceControlModule* SourceControl = ISourceControlModule::GetPtr())
	{
		SourceControl->UnregisterProviderChanged(ProviderChangedHandle);
		SourceControl->SetProvider(ProxiedSourceControlProvider->GetName());
	}
}

void FScopedSourceControlOverride::SetRootProvider(ISourceControlProvider& InNewProvider)
{
	const TGuardValue<bool> ReentrancyGuard(bHandlingProviderChanges, true);
	DesiredRootProvider = &InNewProvider;
	ISourceControlModule& SourceControl = ISourceControlModule::Get();
	SourceControl.SetProvider(DesiredRootProvider->GetName());
}

void FScopedSourceControlOverride::HandleProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	// If we are currently setting ourselves as the provider
	if (bHandlingProviderChanges)
	{
		return;
	}
	const TGuardValue<bool> ReentrancyGuard(bHandlingProviderChanges, true);
	
	// if we receive this event we should be installed as the current provider, if we aren't resetting ourselves
	check(&OldProvider == DesiredRootProvider);
	ProxiedSourceControlProvider = &NewProvider;
	OnProxiedProviderChangedDelegate.Broadcast(NewProvider);
	ISourceControlModule::Get().SetProvider(DesiredRootProvider->GetName());
}
}
