// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "Misc/NotNull.h"
#include "Templates/UnrealTemplate.h"

class ISourceControlProvider;

namespace UE::FileSandboxCore
{
DECLARE_MULTICAST_DELEGATE_OneParam(FSourceControlProviderEvent, ISourceControlProvider&);
	
/** Forces the global ISourceControlProvider to always be the passed in one, even when the user manually changes the source control provider. */
class FScopedSourceControlOverride : public FNoncopyable
{
public:
	
	explicit FScopedSourceControlOverride(ISourceControlProvider& InInitialProvider UE_LIFETIMEBOUND);
	~FScopedSourceControlOverride();
	
	/** Changes the root source control provider again, as if ~FScopedSourceControlOverride() and FScopedSourceControlOverride were called again. */
	void SetRootProvider(ISourceControlProvider& InNewProvider);
	
	/** @return The real source control provider that will be restored to. */
	ISourceControlProvider& GetProxiedSourceControlProvider() const { return *ProxiedSourceControlProvider; }
	
	/** Invoked when the underlying source control provider changes, e.g. when the user changes the source control settings. */
	FSourceControlProviderEvent& OnProxiedProviderChanged() { return OnProxiedProviderChangedDelegate; }

private:
	
	/** The source control provider that should remain the root source control provider. */
	TNotNull<ISourceControlProvider*> DesiredRootProvider;
	/** The provider to restore to when the scope ends. */
	TNotNull<ISourceControlProvider*> ProxiedSourceControlProvider;
	
	/** Reentry guard when handling provider changes. */
	bool bHandlingProviderChanges = false;
	
	/** Delegate handle for provider changes. */
	const FDelegateHandle ProviderChangedHandle;
	
	/** Invoked when the underlying source control provider changes */
	FSourceControlProviderEvent OnProxiedProviderChangedDelegate;
	
	/** Delegate to handle provider change and change our underlying provider. */
	void HandleProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);
};
}

