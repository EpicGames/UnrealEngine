// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Data/ManifestData.h"
#include "Data/MountPointInfo.h"
#include "Utils/MountPointUtils.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace UE::FileSandboxCore
{
BEGIN_DEFINE_SPEC(FMountPointUtilsSpec, "Plugins.FileSandboxCore.MountPointUtils", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	/** Mount points "as they were saved in the manifest" — i.e. before the project was moved. */
	TArray<FFileSandboxCore_MountPoint> ManifestMountPoints;
	/** Mount points "as they are now" — i.e. after the project was moved. */
	TArray<FFileSandboxCore_MountPoint> CurrentMountPoints;
END_DEFINE_SPEC(FMountPointUtilsSpec);

void FMountPointUtilsSpec::Define()
{
	BeforeEach([this]
	{
		// Old project layout, as captured in a manifest. Mount RootPaths are stored with a trailing
		// directory separator, matching the convention from FPackageName::LongPackageNameToFilename.
		ManifestMountPoints.Reset();
		ManifestMountPoints.Emplace(TEXT("/Game/"),     TEXT("C:/MyProject/Content/"));
		ManifestMountPoints.Emplace(TEXT("/Engine/"),   TEXT("C:/UE/Engine/Content/"));
		ManifestMountPoints.Emplace(TEXT("/MyPlugin/"), TEXT("C:/MyProject/Plugins/MyPlugin/Content/"));

		// New project layout, after the user moved the .uproject.
		CurrentMountPoints.Reset();
		CurrentMountPoints.Emplace(TEXT("/Game/"),     TEXT("D:/NewLocation/MyProject/Content/"));
		CurrentMountPoints.Emplace(TEXT("/Engine/"),   TEXT("D:/UE/Engine/Content/"));
		CurrentMountPoints.Emplace(TEXT("/MyPlugin/"), TEXT("D:/NewLocation/MyProject/Plugins/MyPlugin/Content/"));
	});

	Describe("MigrateMountPoint", [this]
	{
		It("rebases a file path onto the current mount with the same MountName", [this]
		{
			const TArray<FString> Inputs =
			{
				TEXT("C:/MyProject/Content/MyBlueprint.uasset"),
				TEXT("C:/UE/Engine/Content/Core/Foo.uasset"),
				TEXT("C:/MyProject/Plugins/MyPlugin/Content/Sub/Asset.uasset"),
			};
			const TArray<FString> Expected =
			{
				TEXT("D:/NewLocation/MyProject/Content/MyBlueprint.uasset"),
				TEXT("D:/UE/Engine/Content/Core/Foo.uasset"),
				TEXT("D:/NewLocation/MyProject/Plugins/MyPlugin/Content/Sub/Asset.uasset"),
			};

			for (int32 Index = 0; Index < Inputs.Num(); ++Index)
			{
				const TOptional<FString> Migrated = MigrateMountPoint(Inputs[Index], ManifestMountPoints, CurrentMountPoints);
				const FString Description = FString::Printf(TEXT("[%d] %s"), Index, *Inputs[Index]);
				if (TestTrue(Description + TEXT(" matched"), Migrated.IsSet()))
				{
					TestEqual(*Description, *Migrated, Expected[Index]);
				}
			}
		});

		It("rebases a path that is exactly a mount root", [this]
		{
			// A bare RootPath (no relative tail) is just the mount root itself.
			const TOptional<FString> Migrated = MigrateMountPoint(
				TEXT("C:/MyProject/Content/"),
				ManifestMountPoints,
				CurrentMountPoints
				);
			if (TestTrue(TEXT("matched"), Migrated.IsSet()))
			{
				TestEqual(TEXT("rebased"), *Migrated, FString(TEXT("D:/NewLocation/MyProject/Content/")));
			}
		});

		It("is case-insensitive on the prefix match", [this]
		{
			const TOptional<FString> Migrated = MigrateMountPoint(
				TEXT("c:/myproject/content/MyBlueprint.uasset"),
				ManifestMountPoints,
				CurrentMountPoints
				);
			if (TestTrue(TEXT("matched"), Migrated.IsSet()))
			{
				// The relative tail is preserved verbatim from the input.
				TestEqual(TEXT("rebased"), *Migrated, FString(TEXT("D:/NewLocation/MyProject/Content/MyBlueprint.uasset")));
			}
		});

		It("returns unset when no manifest mount is a prefix of the input", [this]
		{
			const TArray<FString> Inputs =
			{
				TEXT("C:/SomeOtherDir/Foo.uasset"),
				TEXT(""),
				TEXT("C:/MyProject"), // shorter than any RootPath in the manifest
			};

			for (const FString& Input : Inputs)
			{
				const TOptional<FString> Migrated = MigrateMountPoint(Input, ManifestMountPoints, CurrentMountPoints);
				TestFalse(FString::Printf(TEXT("no match for '%s'"), *Input), Migrated.IsSet());
			}
		});

		It("does not falsely match across non-separator boundaries", [this]
		{
			// "C:/MyProject/Content" must not be treated as a prefix of "C:/MyProject/ContentExtra/...".
			const TOptional<FString> Migrated = MigrateMountPoint(
				TEXT("C:/MyProject/ContentExtra/Foo.uasset"),
				ManifestMountPoints,
				CurrentMountPoints
				);
			TestFalse(TEXT("rejected"), Migrated.IsSet());
		});

		It("picks the longest matching mount when mounts are nested", [this]
		{
			// Add a parent mount that also prefixes the plugin path. The plugin mount is the longer
			// match and must win, otherwise the path would be rebased through the parent mount.
			ManifestMountPoints.Emplace(TEXT("/Project/"), TEXT("C:/MyProject/Plugins/"));
			CurrentMountPoints.Emplace(TEXT("/Project/"), TEXT("D:/NewLocation/MyProject/Plugins/"));

			const TOptional<FString> Migrated = MigrateMountPoint(
				TEXT("C:/MyProject/Plugins/MyPlugin/Content/Asset.uasset"),
				ManifestMountPoints,
				CurrentMountPoints
				);
			if (TestTrue(TEXT("matched"), Migrated.IsSet()))
			{
				TestEqual(
					TEXT("longest match wins"),
					*Migrated,
					FString(TEXT("D:/NewLocation/MyProject/Plugins/MyPlugin/Content/Asset.uasset"))
					);
			}
		});

		It("returns unset when the matched manifest mount has no current counterpart", [this]
		{
			// The manifest still has /MyPlugin/ but the current mount points no longer include it
			// (e.g. plugin was uninstalled). The path cannot be rebased.
			CurrentMountPoints.RemoveAll([](const FFileSandboxCore_MountPoint& InMount)
			{
				return InMount.MountName == TEXT("/MyPlugin/");
			});

			const TOptional<FString> Migrated = MigrateMountPoint(
				TEXT("C:/MyProject/Plugins/MyPlugin/Content/Asset.uasset"),
				ManifestMountPoints,
				CurrentMountPoints
				);
			TestFalse(TEXT("no current mount for /MyPlugin/"), Migrated.IsSet());
		});
	});

	Describe("MigrateMountPoints", [this]
	{
		It("updates each manifest mount's RootPath from its current counterpart", [this]
		{
			MigrateMountPoints(ManifestMountPoints, CurrentMountPoints);

			const TArray<FString> ExpectedRootPaths =
			{
				TEXT("D:/NewLocation/MyProject/Content/"),
				TEXT("D:/UE/Engine/Content/"),
				TEXT("D:/NewLocation/MyProject/Plugins/MyPlugin/Content/"),
			};

			if (TestEqual(TEXT("count"), ManifestMountPoints.Num(), ExpectedRootPaths.Num()))
			{
				for (int32 Index = 0; Index < ManifestMountPoints.Num(); ++Index)
				{
					TestEqual(
						*FString::Printf(TEXT("[%d] %s RootPath"), Index, *ManifestMountPoints[Index].MountName),
						ManifestMountPoints[Index].RootPath,
						ExpectedRootPaths[Index]
						);
				}
			}
		});

		It("preserves MountName for each entry", [this]
		{
			const TArray<FString> ExpectedNames =
			{
				TEXT("/Game/"),
				TEXT("/Engine/"),
				TEXT("/MyPlugin/"),
			};

			MigrateMountPoints(ManifestMountPoints, CurrentMountPoints);

			if (TestEqual(TEXT("count"), ManifestMountPoints.Num(), ExpectedNames.Num()))
			{
				for (int32 Index = 0; Index < ManifestMountPoints.Num(); ++Index)
				{
					TestEqual(
						*FString::Printf(TEXT("[%d] MountName"), Index),
						ManifestMountPoints[Index].MountName,
						ExpectedNames[Index]
						);
				}
			}
		});

		It("leaves mount points with no current counterpart untouched", [this]
		{
			// /RemovedPlugin/ exists only in the manifest. The header contract says unmatched
			// mount points are left as they were.
			ManifestMountPoints.Emplace(TEXT("/RemovedPlugin/"), TEXT("C:/MyProject/Plugins/RemovedPlugin/Content/"));

			MigrateMountPoints(ManifestMountPoints, CurrentMountPoints);

			const FFileSandboxCore_MountPoint* Removed = ManifestMountPoints.FindByPredicate(
				[](const FFileSandboxCore_MountPoint& InMount)
				{
					return InMount.MountName == TEXT("/RemovedPlugin/");
				});
			if (TestNotNull(TEXT("still present"), Removed))
			{
				TestEqual(
					TEXT("RootPath unchanged"),
					Removed->RootPath,
					FString(TEXT("C:/MyProject/Plugins/RemovedPlugin/Content/"))
					);
			}
		});
	});

	Describe("MigrateManifestFilePaths", [this]
	{
		It("rebases manifest file paths and mount points using the engine's current mounts", [this]
		{
			// Pick whatever mount the engine currently has registered as the test subject —
			// the migration logic itself doesn't care which mount name is used.
			FFileSandboxCore_MountPointsInfo CurrentInfo;
			CurrentInfo.Initialize();
			if (!TestFalse(TEXT("engine has at least one registered mount"), CurrentInfo.MountPoints.IsEmpty()))
			{
				return;
			}
			const FString MountName  = CurrentInfo.MountPoints[0].MountName;
			const FString CurrentRoot = CurrentInfo.MountPoints[0].RootPath;

			// Build a manifest whose mount points reference a stale RootPath, and whose file
			// paths live under that stale RootPath.
			const FString StaleRoot = TEXT("X:/Stale/Mount");

			FFileSandboxCore_ManifestData Manifest;
			Manifest.MountPoints.MountPoints.Emplace(MountName, StaleRoot);
			Manifest.ModifiedFiles.Add(FPaths::Combine(StaleRoot, TEXT("Modified.uasset")));
			Manifest.AddedFiles.Add(FPaths::Combine(StaleRoot, TEXT("Added.uasset")));

			const FDateTime DeletedTimestamp(2024, 1, 1);
			Manifest.DeletedFiles.Add(FPaths::Combine(StaleRoot, TEXT("Deleted.uasset")), DeletedTimestamp);

			MigrateManifestFilePaths(Manifest);

			// File paths must now live under the current mount root, not the stale one.
			const FString ExpectedModified = FPaths::ConvertRelativePathToFull(FPaths::Combine(CurrentRoot, TEXT("Modified.uasset")));
			const FString ExpectedAdded    = FPaths::ConvertRelativePathToFull(FPaths::Combine(CurrentRoot, TEXT("Added.uasset")));
			const FString ExpectedDeleted  = FPaths::ConvertRelativePathToFull(FPaths::Combine(CurrentRoot, TEXT("Deleted.uasset")));

			TestTrue(TEXT("ModifiedFiles rebased"), Manifest.ModifiedFiles.GetFiles().Contains(ExpectedModified));
			TestTrue(TEXT("AddedFiles rebased"),    Manifest.AddedFiles.GetFiles().Contains(ExpectedAdded));
			TestTrue(TEXT("DeletedFiles rebased"),  Manifest.DeletedFiles.GetFiles().Contains(ExpectedDeleted));

			// The timestamp recorded against the deleted file must survive migration.
			const TOptional<FDateTime> RetrievedTimestamp = Manifest.DeletedFiles.GetTimestamp(ExpectedDeleted);
			if (TestTrue(TEXT("timestamp present"), RetrievedTimestamp.IsSet()))
			{
				TestEqual(TEXT("timestamp preserved"), *RetrievedTimestamp, DeletedTimestamp);
			}

			// The manifest's own mount points must now reflect the current RootPath.
			const FFileSandboxCore_MountPoint* MigratedMount = Manifest.MountPoints.MountPoints.FindByPredicate(
				[&MountName](const FFileSandboxCore_MountPoint& InMount)
				{
					return InMount.MountName == MountName;
				});
			if (TestNotNull(TEXT("mount still present"), MigratedMount))
			{
				TestEqual(TEXT("mount RootPath updated to current"), MigratedMount->RootPath, CurrentRoot);
			}
		});

		It("leaves paths that match no mount untouched", [this]
		{
			// Build a manifest whose mount points have no counterpart in the engine. File paths
			// under those mounts cannot be rebased and must be left alone.
			FFileSandboxCore_ManifestData Manifest;
			Manifest.MountPoints.MountPoints.Emplace(TEXT("/MountThatDoesNotExist/"), TEXT("X:/Foreign/Root"));
			const FString UnchangedPath = FPaths::ConvertRelativePathToFull(TEXT("X:/Foreign/Root/Asset.uasset"));
			Manifest.ModifiedFiles.Add(UnchangedPath);

			MigrateManifestFilePaths(Manifest);

			TestTrue(
				TEXT("Path with no matching mount is preserved"),
				Manifest.ModifiedFiles.GetFiles().Contains(UnchangedPath)
				);
		});
	});
}
}
#endif