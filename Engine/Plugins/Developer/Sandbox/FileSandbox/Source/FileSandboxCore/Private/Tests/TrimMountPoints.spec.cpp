// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Data/ManifestData.h"
#include "Data/MountPointInfo.h"
#include "Misc/Paths.h"
#include "Utils/MountPointUtils.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace UE::FileSandboxCore
{
namespace TrimToReferencedMountPointsTests
{
/** Returns the MountName of every mount point left in the manifest, in order. */
static TArray<FString> CollectMountNames(const FFileSandboxCore_ManifestData& InManifest)
{
	TArray<FString> Names;
	Names.Reserve(InManifest.MountPoints.MountPoints.Num());
	for (const FFileSandboxCore_MountPoint& MountPoint : InManifest.MountPoints.MountPoints)
	{
		Names.Add(MountPoint.MountName);
	}
	return Names;
}
}

BEGIN_DEFINE_SPEC(FTrimToReferencedMountPointsSpec, "Plugins.FileSandboxCore.TrimToReferencedMountPoints", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	FFileSandboxCore_ManifestData Manifest;
END_DEFINE_SPEC(FTrimToReferencedMountPointsSpec);

void FTrimToReferencedMountPointsSpec::Define()
{
	BeforeEach([this]
	{
		// A manifest with three registered mount points but no file references yet.
		// Each test adds files (or doesn't) to drive which mount points should survive.
		Manifest = FFileSandboxCore_ManifestData{};
		Manifest.MountPoints.MountPoints.Emplace(TEXT("/Game/"),     TEXT("C:/MyProject/Content"));
		Manifest.MountPoints.MountPoints.Emplace(TEXT("/Engine/"),   TEXT("C:/UE/Engine/Content"));
		Manifest.MountPoints.MountPoints.Emplace(TEXT("/MyPlugin/"), TEXT("C:/MyProject/Plugins/MyPlugin/Content"));
	});

	It("removes mount points that are not referenced by any file", [this]
	{
		// Only /Game/ is referenced. /Engine/ and /MyPlugin/ have no files under their RootPaths
		// and must be trimmed.
		const TArray<FString> InputAddedFiles =
		{
			FPaths::Combine(TEXT("C:/MyProject/Content"), TEXT("MyBlueprint.uasset")),
		};
		const TArray<FString> ExpectedRemainingMountNames =
		{
			TEXT("/Game/"),
		};

		for (const FString& Path : InputAddedFiles)
		{
			Manifest.AddedFiles.Add(Path);
		}

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});

	It("keeps a mount referenced only by AddedFiles", [this]
	{
		const TArray<FString> InputAddedFiles =
		{
			FPaths::Combine(TEXT("C:/MyProject/Plugins/MyPlugin/Content"), TEXT("Asset.uasset")),
		};
		const TArray<FString> ExpectedRemainingMountNames =
		{
			TEXT("/MyPlugin/"),
		};

		for (const FString& Path : InputAddedFiles)
		{
			Manifest.AddedFiles.Add(Path);
		}

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});

	It("keeps a mount referenced only by ModifiedFiles", [this]
	{
		const TArray<FString> InputModifiedFiles =
		{
			FPaths::Combine(TEXT("C:/MyProject/Plugins/MyPlugin/Content"), TEXT("Asset.uasset")),
		};
		const TArray<FString> ExpectedRemainingMountNames =
		{
			TEXT("/MyPlugin/"),
		};

		for (const FString& Path : InputModifiedFiles)
		{
			Manifest.ModifiedFiles.Add(Path);
		}

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});

	It("keeps a mount referenced only by DeletedFiles", [this]
	{
		const TArray<FString> InputDeletedFiles =
		{
			FPaths::Combine(TEXT("C:/MyProject/Plugins/MyPlugin/Content"), TEXT("Asset.uasset")),
		};
		const TArray<FString> ExpectedRemainingMountNames =
		{
			TEXT("/MyPlugin/"),
		};

		for (const FString& Path : InputDeletedFiles)
		{
			Manifest.DeletedFiles.Add(Path);
		}

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});

	It("keeps all mounts when each is referenced by a different bucket", [this]
	{
		// One reference per bucket, each pointing at a different mount. All three should survive.
		const TArray<FString> InputAddedFiles =
		{
			FPaths::Combine(TEXT("C:/MyProject/Content"), TEXT("Added.uasset")),
		};
		const TArray<FString> InputModifiedFiles =
		{
			FPaths::Combine(TEXT("C:/UE/Engine/Content"), TEXT("Modified.uasset")),
		};
		const TArray<FString> InputDeletedFiles =
		{
			FPaths::Combine(TEXT("C:/MyProject/Plugins/MyPlugin/Content"), TEXT("Deleted.uasset")),
		};
		const TArray<FString> ExpectedRemainingMountNames =
		{
			TEXT("/Game/"),
			TEXT("/Engine/"),
			TEXT("/MyPlugin/"),
		};

		for (const FString& Path : InputAddedFiles)    { Manifest.AddedFiles.Add(Path); }
		for (const FString& Path : InputModifiedFiles) { Manifest.ModifiedFiles.Add(Path); }
		for (const FString& Path : InputDeletedFiles)  { Manifest.DeletedFiles.Add(Path); }

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});

	It("removes all mount points when no files are referenced", [this]
	{
		// No files added in any bucket — every mount point must be trimmed.
		const TArray<FString> ExpectedRemainingMountNames;

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});

	It("does not falsely keep a mount whose RootPath is a non-separator prefix of a file", [this]
	{
		// "C:/MyProject/Content" must NOT be treated as a prefix of "C:/MyProject/ContentExtra/...",
		// so /Game/ should be trimmed even though the string-level prefix matches.
		Manifest.MountPoints.MountPoints.Emplace(TEXT("/ContentExtra/"), TEXT("C:/MyProject/ContentExtra"));

		const TArray<FString> InputAddedFiles =
		{
			FPaths::Combine(TEXT("C:/MyProject/ContentExtra"), TEXT("Asset.uasset")),
		};
		const TArray<FString> ExpectedRemainingMountNames =
		{
			TEXT("/ContentExtra/"),
		};

		for (const FString& Path : InputAddedFiles)
		{
			Manifest.AddedFiles.Add(Path);
		}

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});

	It("is a no-op when the manifest has no mount points", [this]
	{
		// Files are present but the mount point list is empty — nothing to trim, no crash.
		Manifest.MountPoints.MountPoints.Reset();
		Manifest.AddedFiles.Add(FPaths::Combine(TEXT("C:/MyProject/Content"), TEXT("Asset.uasset")));

		const TArray<FString> ExpectedRemainingMountNames;

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});

	It("keeps mount points whose RootPath ends with a directory separator", [this]
	{
		// Some pipelines store RootPath with a trailing separator. Ensure that variant still matches
		// the file paths recorded against it.
		Manifest.MountPoints.MountPoints.Reset();
		Manifest.MountPoints.MountPoints.Emplace(TEXT("/Game/"), TEXT("C:/MyProject/Content/"));

		const TArray<FString> InputAddedFiles =
		{
			FPaths::Combine(TEXT("C:/MyProject/Content"), TEXT("MyBlueprint.uasset")),
		};
		const TArray<FString> ExpectedRemainingMountNames =
		{
			TEXT("/Game/"),
		};

		for (const FString& Path : InputAddedFiles)
		{
			Manifest.AddedFiles.Add(Path);
		}

		TrimToReferencedMountPoints(Manifest);

		TestEqual(TEXT("remaining mount names"), TrimToReferencedMountPointsTests::CollectMountNames(Manifest), ExpectedRemainingMountNames);
	});
}
}
#endif