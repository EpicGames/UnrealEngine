// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/Manager/LoadSandboxError.h"

#include "Containers/UnrealString.h"

namespace UE::FileSandboxCore
{
FString LexToString(ELoadSandboxLoadErrorReason InReason)
{
	static_assert(static_cast<int32>(ELoadSandboxLoadErrorReason::Count) == 6, "Update this switch");
	switch (InReason)
	{
	case ELoadSandboxLoadErrorReason::InvalidDirectory: return TEXT("InvalidDirectory");
	case ELoadSandboxLoadErrorReason::InvalidName: return TEXT("InvalidName");
	case ELoadSandboxLoadErrorReason::IOError: return TEXT("IOError");
	case ELoadSandboxLoadErrorReason::CannotLeaveSandbox: return TEXT("CannotLeaveSandbox");
	case ELoadSandboxLoadErrorReason::IncompatibleVersion: return TEXT("IncompatibleVersion");
	case ELoadSandboxLoadErrorReason::Unspecified: return TEXT("Unspecified");
	default: checkNoEntry(); return TEXT("Unknown");
	}
}
}
