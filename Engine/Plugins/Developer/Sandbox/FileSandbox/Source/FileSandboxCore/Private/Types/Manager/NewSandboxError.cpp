// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/Manager/NewSandboxError.h"

#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"

namespace UE::FileSandboxCore
{
FString LexToString(ENewSandboxErrorReason InReason)
{
	static_assert(static_cast<int32>(ENewSandboxErrorReason::Count) == 4, "Update this switch");
	switch (InReason)
	{
	case ENewSandboxErrorReason::UnsuitablePath: return TEXT("UnsuitablePath");
	case ENewSandboxErrorReason::IOError: return TEXT("IOError");
	case ENewSandboxErrorReason::CannotLeaveSandbox: return TEXT("CannotLeaveSandbox");
	case ENewSandboxErrorReason::Unspecified: return TEXT("Unspecified");
	default: checkNoEntry(); return TEXT("Unknown");
	}
}
}
