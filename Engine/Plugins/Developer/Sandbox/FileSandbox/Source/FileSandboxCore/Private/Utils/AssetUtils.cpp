// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetUtils.h"

#include "HAL/FileManager.h"
#include "Interface/IPackageReloadHandler.h"
#include "Modules/ModuleManager.h"
#include "Types/Package/HotReloadPackageArgs.h"
#include "Types/Package/PurgePackageArgs.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Utils/PackageSandboxUtils.h"

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"
#include "Watcher/DirectoryWatcherUtils.h"
#endif

#if WITH_EDITOR
#include "AssetRegistry/IAssetRegistry.h"
#endif

namespace UE::FileSandboxCore
{
void ReplayChanges(
	IPackageReloadHandler& InReloaderHandler, 
	TConstArrayView<FString> DeletedFiles, TConstArrayView<FString> ModifiedFiles, TConstArrayView<FString> AddedFiles,
	ESandboxPackageReloadPhase InPhase
	)
{
	// Causes any open editors to refresh
	TArray<FName> PackagesPendingHotReload;
	for (const FString& Modified : ModifiedFiles)
	{
		FString PackageName;
		if (FPackageName::TryConvertFilenameToLongPackageName(Modified, PackageName))
		{
			PackagesPendingHotReload.Add(FName(*PackageName));
		}
	}

	// This closes any open editors and removes the files from the content browser
	TArray<FName> PackagesPendingPurge;
	IFileManager& FileManager = IFileManager::Get();
	for (const FString& Deleted : DeletedFiles)
	{
		FString PackageName;
		if (FPackageName::TryConvertFilenameToLongPackageName(Deleted, PackageName))
		{
			PackagesPendingPurge.Add(FName(*PackageName));
			FileManager.Delete(*Deleted);
		}
	}

#if WITH_EDITOR
	// Force the Asset Registry to re-scan files the sandbox introduced or modified. World Partition's
	// UActorDescContainer::Initialize calls AssetRegistry::ScanSynchronous on the external actors path
	// without the ForceRescan flag, so if AR already scanned that directory at editor startup (before
	// the sandbox overlay was active), it treats the call as a no-op and the sandbox's added/modified
	// external actor packages stay invisible to WP. Explicit ScanFilesSynchronous with ForceRescan
	// ensures AR picks them up before the persistent level reload below queries it.
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		TArray<FString> FilesToRescan;
		FilesToRescan.Reserve(AddedFiles.Num() + ModifiedFiles.Num());
		FilesToRescan.Append(AddedFiles.GetData(), AddedFiles.Num());
		FilesToRescan.Append(ModifiedFiles.GetData(), ModifiedFiles.Num());
		if (!FilesToRescan.IsEmpty())
		{
			AssetRegistry->ScanFilesSynchronous(FilesToRescan, /*bForceRescan=*/true);
		}
	}
#endif

	// In a World Partition level, actor add/remove/modify changes live in per-actor external packages and do
	// not dirty the persistent level itself. Force a reload of the persistent level so the WP runtime picks
	// up the sandbox's added/modified/deleted actor packages when the sandbox is loaded.
	AppendExternalPersistentLevelForReload(PackagesPendingHotReload, PackagesPendingPurge);

	InReloaderHandler.HotReloadPackages(FHotReloadPackageArgs(InPhase, PackagesPendingHotReload));
	InReloaderHandler.PurgePackages(FPurgePackageArgs(InPhase, PackagesPendingPurge));

	// This is needed to make added files show up in the content browser
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
	TArray<FFileChangeData> FileChanges;
	Algo::Transform(AddedFiles, FileChanges, [](const FString& File){ return FFileChangeData(File, FFileChangeData::FCA_Added); });
	if (FDirectoryWatcherModule* DirectoryWatcherModule = GetDirectoryWatcherModuleIfLoaded())
	{
		DirectoryWatcherModule->RegisterExternalChanges(FileChanges);
	}
#endif

	SynchronizeAssetRegistry();
}
	
void SynchronizeAssetRegistry()
{
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
	IDirectoryWatcher* DirectoryWatcher = GetDirectoryWatcherIfLoaded();
	if (!DirectoryWatcher)
	{
		return;
	}

	DirectoryWatcher->Tick(0.0f);
#endif // WITH_EDITOR
}

void FlushPackageLoading(const FString& InPackageName, bool bForceBulkDataLoad)
{
	UPackage* ExistingPackage = FindPackage(nullptr, *InPackageName);
	if (ExistingPackage)
	{
		if (!ExistingPackage->IsFullyLoaded())
		{
			FlushAsyncLoading();
			ExistingPackage->FullyLoad();
		}

		if (bForceBulkDataLoad)
		{
			ResetLoaders(ExistingPackage);
		}
		else if (ExistingPackage->GetLinker())
		{
			ExistingPackage->GetLinker()->Detach();
		}
	}
}

bool FlushPackageFile(const FString& InFilename, FName* OutPackageName, bool bForceLoad)
{
	FString PackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(InFilename, PackageName))
	{
		FlushPackageLoading(PackageName, bForceLoad);
		if (OutPackageName)
		{
			*OutPackageName = *PackageName;
		}
		return true;
	}
	return false;
}
}
