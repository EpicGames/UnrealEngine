// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Types/Package/SandboxPackageReloadPhase.h"

class FDirectoryWatcherModule;
class FName;
class FString;
class IDirectoryWatcher;

namespace UE::FileSandboxCore
{
class IPackageReloadHandler;

/** Utility for applying changes from manifest file to the current IPlatformFile. */
void ReplayChanges(
	IPackageReloadHandler& InReloaderHandler, 
	TConstArrayView<FString> DeletedFiles, TConstArrayView<FString> ModifiedFiles, TConstArrayView<FString> AddedFiles,
	ESandboxPackageReloadPhase InPhase = ESandboxPackageReloadPhase::Startup
	);
	
/** Refreshes the assets registry after making underlying file changes. */
void SynchronizeAssetRegistry();

/** Finishes loading the package, if currently async loading, and detaches the linker. */
void FlushPackageLoading(const FString& InPackageName, bool bForceBulkDataLoad = true);
/** Alternate variant that flushes package loading by filename instead. */
bool FlushPackageFile(const FString& InFilename, FName* OutPackageName = nullptr, bool bForceLoad = true);
}
