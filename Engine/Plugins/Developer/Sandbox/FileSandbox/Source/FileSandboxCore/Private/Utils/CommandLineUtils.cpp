// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommandLineUtils.h"

#include "CoreGlobals.h"
#include "IFileSandboxCoreModule.h"
#include "ISandboxManager.h"
#include "LogFileSandbox.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Types/Manager/DeleteSandboxByDirectoryArgs.h"
#include "Types/Manager/DeleteSandboxByNameArgs.h"
#include "Types/Manager/DeleteSandboxResult.h"
#include "Types/Manager/NewSandboxArgs.h"
#include "Utils/BuiltInTags.h"

namespace UE::FileSandboxCore
{
namespace CommandDetail
{
/** Deletes the sandbox with the given name. */
const TCHAR* DeleteSandboxByName = TEXT("-DELETESANDBOXBYNAME=");
/** Deletes the sandbox with the given path. */
const TCHAR* DeleteSandboxByPath = TEXT("-DELETESANDBOXBYPATH=");

/** Starts a new sandbox with the given name. */
const TCHAR* StartWithSandbox = TEXT("-STARTWITHSANDBOX=");
/** Optional description to give the new sandbox. */
const TCHAR* SandboxDescription = TEXT("-SANDBOXDESCRIPTION=");
/** Optional base path to place the sandbox in. Uses the default directory if unspecified. */
const TCHAR* SandboxBaseDirectory = TEXT("-SANDBOXBASEDIRECTORY=");

/** Sets the folder in which new sandboxes are placed by default. */
const TCHAR* DefaultSandboxDirectory = TEXT("-DEFAULTSANDBOXDIRECTORY=");

static void HandleDeleteNamedSandbox()
{
	FString Name;
	if (FParse::Value(FCommandLine::Get(), DeleteSandboxByName, Name))
	{
		ISandboxManager& SandboxManager = IFileSandboxCoreModule::Get().GetSandboxManager();
		const FDeleteSandboxResult DeleteResult = SandboxManager.DeleteNamedSandbox(FDeleteSandboxByNameArgs(Name));
		
		UE_CLOGF(DeleteResult.IsSuccess(), LogFileSandbox, Log, "Deleted named sandbox '%ls' from command line", *Name);
		UE_CLOGF(!DeleteResult.IsSuccess(), LogFileSandbox, Error, "Failed to delete named sandbox '%ls from command line'", *Name);
	}
}

static void HandleDeleteSandboxByPath()
{
	FString Path;
	if (FParse::Value(FCommandLine::Get(), DeleteSandboxByPath, Path))
	{
		ISandboxManager& SandboxManager = IFileSandboxCoreModule::Get().GetSandboxManager();
		const FDeleteSandboxResult DeleteResult = SandboxManager.DeleteSandbox(
			FDeleteSandboxByDirectoryArgs(Path)
			);
		
		UE_CLOGF(DeleteResult.IsSuccess(), LogFileSandbox, Log, "Deleted sandbox by path '%ls' from command line", *Path);
		UE_CLOGF(!DeleteResult.IsSuccess(), LogFileSandbox, Error, "Failed sandbox by path '%ls from command line'", *Path);
	}
}

static void HandleStartWithSandbox()
{
	FString Name;
	if (!FParse::Value(FCommandLine::Get(), StartWithSandbox, Name))
	{
		return;
	}
	
	if (Name.IsEmpty())
	{
		UE_LOGF(LogFileSandbox, Error, "Cannot start sandbox  from command line because no name was specified");
		return;
	}
	
	FString Description = TEXT("Command Line Sandbox");
	FParse::Value(FCommandLine::Get(), SandboxDescription, Description);
	
	FString BaseDirectory = GetBaseSandboxDirectory();
	FParse::Value(FCommandLine::Get(), SandboxBaseDirectory, BaseDirectory);
		
	FNewSandboxArgs Args(Name, Description);
	Args.SandboxBasePath = BaseDirectory;
	Args.MetaData.Tags.Add(Tag_CommandLineSandbox);
	ISandboxManager& SandboxManager = IFileSandboxCoreModule::Get().GetSandboxManager();
	const FNewSandboxResult Result = SandboxManager.CreateNewSandbox(Args);
		
	UE_CLOGF(Result.HasValue(), LogFileSandbox, Log, "Started sandbox '%ls' from command line", *Name);
	UE_CLOGF(Result.HasError(), LogFileSandbox, Error, "Failed to start sandbox '%ls' from command line'", *Name);
}
}

void RegisterStartupCommandLineDelegate()
{
	if (!IsRunningCommandlet())
	{
		FCoreDelegates::OnFEngineLoopInitComplete.AddStatic(&ParseStartupCommandLine);
	}
}

void ParseStartupCommandLine()
{
	CommandDetail::HandleDeleteNamedSandbox();
	CommandDetail::HandleDeleteSandboxByPath();
	CommandDetail::HandleStartWithSandbox();
}

FString ParseDefaultSandboxDirectory()
{
	FString CommandLineDirectory;
	const bool bHasOverride = FParse::Value(FCommandLine::Get(), CommandDetail::DefaultSandboxDirectory, CommandLineDirectory);
	return bHasOverride && FPaths::ValidatePath(CommandLineDirectory) 
		? CommandLineDirectory 
		: FString();
}
}