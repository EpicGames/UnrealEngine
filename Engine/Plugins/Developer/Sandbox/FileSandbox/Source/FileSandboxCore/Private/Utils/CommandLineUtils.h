// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FString;

namespace UE::FileSandboxCore
{
/** Parses the command line once engine start up is done; this is only done if command line arguments should be parsed, i.e. not running commandlet. */
void RegisterStartupCommandLineDelegate();

/** Parses command line commands that are intended for engine startup. */
void ParseStartupCommandLine();

/** @return The default sandbox directory set via command line, or empty no (valid) directory is set. */
FString ParseDefaultSandboxDirectory();
}
