// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
#include "IDirectoryWatcher.h"
#endif

namespace UE::FileSandboxCore
{
enum class EFileChangeAction : uint8
{
	Added,
	Modified,
	Removed
};

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
/** @return Converts the change action to the equivalent IDirectoryWatcher type. */
FFileChangeData::EFileChangeAction ToDirectoryWatcherAction(EFileChangeAction InAction);
#endif
}

namespace UE::FileSandboxCore
{
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
inline FFileChangeData::EFileChangeAction ToDirectoryWatcherAction(EFileChangeAction InAction)
{
	switch (InAction)
	{
	case EFileChangeAction::Added: return FFileChangeData::FCA_Added;
	case EFileChangeAction::Modified: return FFileChangeData::FCA_Modified;
	case EFileChangeAction::Removed: return FFileChangeData::FCA_Removed;
	default: checkNoEntry(); return FFileChangeData::FCA_Unknown;
	}
}
#endif
}