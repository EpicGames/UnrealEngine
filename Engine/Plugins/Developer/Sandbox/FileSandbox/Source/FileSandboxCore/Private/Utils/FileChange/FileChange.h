// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "EFileChangeAction.h"
#include "Misc/Paths.h"

namespace UE::FileSandboxCore
{
/** Describes a file change. This mirrors FFileChangeData from DirectoryWatchers because that module cannot be built in all runtime configurations. */
struct FFileChange
{
	FString Filename;
	EFileChangeAction Action;
	
	FFileChange(const FString& InFilename, EFileChangeAction InAction)
		: Filename(InFilename)
		, Action(InAction)
	{
		FPaths::MakeStandardFilename(Filename);
	}
	
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
	/** @return Converts this change to the equivalent IDirectoryWatcher type. */
	FFileChangeData ToDirectoryWatcherChange() const
	{
		return FFileChangeData(Filename, ToDirectoryWatcherAction(Action));
	}
#endif
};
}
