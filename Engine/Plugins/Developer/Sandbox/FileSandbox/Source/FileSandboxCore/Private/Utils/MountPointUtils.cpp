// Copyright Epic Games, Inc. All Rights Reserved.

#include "MountPointUtils.h"

#include "Data/ManifestData.h"
#include "Data/MountPointInfo.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"

namespace UE::FileSandboxCore
{
namespace MountPointDetail
{
static const FFileSandboxCore_MountPoint* FindMountPointByName(
	TConstArrayView<FFileSandboxCore_MountPoint> InMountPoints,
	const FString& InMountName
	)
{
	return InMountPoints.FindByPredicate(
		[&InMountName](const FFileSandboxCore_MountPoint& InMountPoint)
		{
			return InMountPoint.MountName == InMountName;
		});
}
}

TOptional<FString> MigrateMountPoint(
	const FString& InPath,
	TConstArrayView<FFileSandboxCore_MountPoint> InManifestMountPoints,
	TConstArrayView<FFileSandboxCore_MountPoint> InCurrentMountPoints
	)
{
	// Prefer the longest matching manifest mount, so nested mounts resolve correctly.
	// Example: InPath = "C:/MyProject/Plugins/MyPlugin/Content/Asset.uasset" and the manifest contains both
	//   /Project/  -> "C:/MyProject/Plugins/"
	//   /MyPlugin/ -> "C:/MyProject/Plugins/MyPlugin/Content/"
	// Both RootPaths are prefixes of InPath, but /MyPlugin/ is the longer (and correct) match.
	const FFileSandboxCore_MountPoint* BestMatch = nullptr;
	FStringView BestRelativePath;
	for (const FFileSandboxCore_MountPoint& ManifestMount : InManifestMountPoints)
	{
		FStringView RelativePath;
		if (FPathViews::TryMakeChildPathRelativeTo(InPath, ManifestMount.RootPath, RelativePath)
			&& (!BestMatch || ManifestMount.RootPath.Len() > BestMatch->RootPath.Len()))
		{
			BestMatch = &ManifestMount;
			BestRelativePath = RelativePath;
		}
	}

	if (!BestMatch)
	{
		return {};
	}

	const FFileSandboxCore_MountPoint* CurrentMount = MountPointDetail::FindMountPointByName(InCurrentMountPoints, BestMatch->MountName);
	if (!CurrentMount)
	{
		return {};
	}

	return FPaths::Combine(CurrentMount->RootPath, BestRelativePath);
}

void MigrateMountPoints(
	TArrayView<FFileSandboxCore_MountPoint> InManifestMountPoints,
	TConstArrayView<FFileSandboxCore_MountPoint> InCurrentMountPoints
	)
{
	for (FFileSandboxCore_MountPoint& ManifestMount : InManifestMountPoints)
	{
		TOptional<FString> MigratedRootPath = MigrateMountPoint(
			ManifestMount.RootPath,
			InManifestMountPoints,
			InCurrentMountPoints
			);
		if (MigratedRootPath.IsSet())
		{
			ManifestMount.RootPath = MoveTemp(*MigratedRootPath);
		}
	}
}

void MigrateManifestFilePaths(FFileSandboxCore_ManifestData& InManifest)
{
	// Snapshot before mutating: file paths in the manifest were recorded against these mount points,
	// and MigrateMountPoints below will overwrite InManifest.MountPoints.
	const TArray<FFileSandboxCore_MountPoint> OriginalManifestMountPoints = InManifest.MountPoints.MountPoints;

	FFileSandboxCore_MountPointsInfo CurrentMountPoints;
	CurrentMountPoints.Initialize();

	const auto MigrateOrKeep = [&OriginalManifestMountPoints, &CurrentMountPoints](const FString& InPath) -> FString
	{
		TOptional<FString> Migrated = MigrateMountPoint(InPath, OriginalManifestMountPoints, CurrentMountPoints.MountPoints);
		return Migrated.IsSet() ? MoveTemp(*Migrated) : InPath;
	};

	{
		const TArray<FString> OldFiles = InManifest.ModifiedFiles.GetFiles();
		InManifest.ModifiedFiles.Empty();
		for (const FString& OldPath : OldFiles)
		{
			InManifest.ModifiedFiles.Add(MigrateOrKeep(OldPath));
		}
	}

	{
		const TArray<FString> OldFiles = InManifest.AddedFiles.GetFiles();
		InManifest.AddedFiles.Empty();
		for (const FString& OldPath : OldFiles)
		{
			InManifest.AddedFiles.Add(MigrateOrKeep(OldPath));
		}
	}

	{
		const bool bTimestampsValid = InManifest.DeletedFiles.AreTimestampsValid();
		const TArray<FString> OldFiles = InManifest.DeletedFiles.GetFiles();
		const TArray<FDateTime> OldTimestamps = InManifest.DeletedFiles.GetTimestamps();
		InManifest.DeletedFiles.Empty();
		for (int32 Index = 0; Index < OldFiles.Num(); ++Index)
		{
			const FString NewPath = MigrateOrKeep(OldFiles[Index]);
			if (bTimestampsValid)
			{
				InManifest.DeletedFiles.Add(NewPath, OldTimestamps[Index]);
			}
			else
			{
				InManifest.DeletedFiles.Add(NewPath);
			}
		}
	}

	MigrateMountPoints(InManifest.MountPoints.MountPoints, CurrentMountPoints.MountPoints);
}

void TrimToReferencedMountPoints(FFileSandboxCore_ManifestData& InManifest)
{
	const auto IsReferencedBy = [](const FFileSandboxCore_MountPoint& InMountPoint, TConstArrayView<FString> InFiles)
	{
		for (const FString& File : InFiles)
		{
			if (FPathViews::IsParentPathOf(InMountPoint.RootPath, File))
			{
				return true;
			}
		}
		return false;
	};

	InManifest.MountPoints.MountPoints.RemoveAll(
		[&InManifest, &IsReferencedBy](const FFileSandboxCore_MountPoint& InMountPoint)
		{
			const bool bReferenced =
				IsReferencedBy(InMountPoint, InManifest.AddedFiles.GetFiles())
				|| IsReferencedBy(InMountPoint, InManifest.ModifiedFiles.GetFiles())
				|| IsReferencedBy(InMountPoint, InManifest.DeletedFiles.GetFiles());
			return !bReferenced;
		});
}
}
