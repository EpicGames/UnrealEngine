// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Data/AbsoluteFilePathArray.h"

struct FFileSandboxCore_ManifestData;
struct FFileSandboxCore_MountPoint;

namespace UE::FileSandboxCore
{
/**
 * Matches a path with any of InManifestMountPoints and tries to migrate it based off of InCurrentMountPoints.
 * @param InPath Path that was in the manifest
 * @param InManifestMountPoints The mount points when the path was saved
 * @param InCurrentMountPoints The current mount points
 * @return The migrated file path, or unset if no mount point could be matched.
 */
TOptional<FString> MigrateMountPoint(
	const FString& InPath, 
	TConstArrayView<FFileSandboxCore_MountPoint> InManifestMountPoints, 
	TConstArrayView<FFileSandboxCore_MountPoint> InCurrentMountPoints
	);

/**
 * Updates FFileSandboxCore_MountPoint::RootPath from the current version.
 * 
 * For example, if
 * - InManifestMountPoints contains MountName = "/Game/" and RootPath = "C:/MyProject/Content"
 * - InCurrentMountPoints contains MountName = "/Game/" and RootPath = C:/NewLocation/MyProject/Content
 * Then the final RootPath for MountName = "/Game/" is "C:/NewLocation/MyProject/Content".
 * 
 * Unmatched mount points are left as they were.
 * 
 * @param InManifestMountPoints The mount points to migrate
 * @param InCurrentMountPoints The current mount points
 */
void MigrateMountPoints(
	TArrayView<FFileSandboxCore_MountPoint> InManifestMountPoints, 
	TConstArrayView<FFileSandboxCore_MountPoint> InCurrentMountPoints
	);

/** 
 * Tries to migrate file paths based off of mount points.
 * 
 * This is relevant for sandboxes that have been imported or if the user has moved the uproject location since the sandbox was created.
 * Old file paths will be matched to the mount points that existed when the manifest was saved.
 * 
 * For example, if
 * - "C:/MyProject/Content/MyBlueprint.uasset" is in the removed files
 * - the uproject has since moved to "D:/NewProjectLocation"
 * Then the removed file will be migrated to "D:/NewProjectLocation/Game/MyBlueprint.uasset"
 */
void MigrateManifestFilePaths(FFileSandboxCore_ManifestData& InManifest);

/** 
 * Removes all mount points that are not being referenced by added, edited, or deleted files.
 * 
 * For example, if
 * - MountsPoints contains 
 *	- MountName = "/Game/" and RootPath = "C:/MyProject/Content"
 *	- MountName = "/MyPlugin/" and RootPath = "C:/MyProject/Plugins/MyPlugin/Content"
 * - Removed files contains "C:/MyProject/Content/MyBlueprint.uasset"
 * then the mount point with MountName = "/MyPlugin/" will be removed from the manifest. 
 */
void TrimToReferencedMountPoints(FFileSandboxCore_ManifestData& InManifest);
}
