// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/PackageSandboxUtils.h"

#include "Engine/GameEngine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameMapsSettings.h"
#include "Misc/MessageDialog.h"
#include "RenderingThread.h"
#include "UObject/LinkerLoad.h"

#if WITH_EDITOR
#include "Editor.h"
#include "FileHelpers.h"
#include "Selection.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif

namespace UE::FileSandboxCore
{
	
UWorld* GetCurrentWorld()
{
	UWorld* CurrentWorld = nullptr;
#if WITH_EDITOR
	if (GIsEditor)
	{
		CurrentWorld = GEditor->GetEditorWorldContext().World();
	}
	else
#endif
		if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			CurrentWorld = GameEngine->GetGameWorld();
		}
	return CurrentWorld;
}

bool ShouldReloadPersistentLevel(UPackage *PackageToReload)
{
	UWorld* CurrentWorld = GetCurrentWorld();
	if (CurrentWorld)
	{
		const TArray<ULevel*> Levels = CurrentWorld->GetLevels();
		for (ULevel* Level : Levels)
		{
			if (Level->GetPackage() == PackageToReload)
			{
				return true;
			}
		}
	}

	return false;
}
	
void PurgePackages(const TConstArrayView<FName>& InPackageNames)
{
	if (InPackageNames.Num() == 0)
	{
		return;
	}

#if WITH_EDITOR
	TArray<UObject*> ObjectsToPurge;
	auto CollectObjectToPurge = [&ObjectsToPurge](UObject* InObject)
	{
		ObjectsToPurge.Add(InObject);
	};

	// Get the current edited map package to check if its going to be purged.
	bool bEditedMapPurged = false;
	UWorld* CurrentWorld = GetCurrentWorld();
	UPackage* EditedMapPackage = CurrentWorld ? CurrentWorld->GetOutermost(): nullptr;

	// Collect any in-memory packages that should be purged and check if we are including the current map in the purge.
	for (const FName& PackageName : InPackageNames)
	{
		UPackage* ExistingPackage = FindPackage(nullptr, *PackageName.ToString());
		if (ExistingPackage)
		{
			// Prevent any message from the editor saying a package is not saved or doesn't exist on disk.
			ExistingPackage->SetDirtyFlag(false);

			CollectObjectToPurge(ExistingPackage);
			ForEachObjectWithPackage(ExistingPackage, [&CollectObjectToPurge](UObject* InObject)
			{
				CollectObjectToPurge(InObject);
				return true;
			});

			bEditedMapPurged |= EditedMapPackage == ExistingPackage;
		}
	}

	// Close editors and deselect after iteration is complete — doing this inside
	// ForEachObjectWithPackage can create new UObjects (toolbar rebuild) which
	// crashes because the UObject hash map is still being iterated.
	for (UObject* Object : ObjectsToPurge)
	{
		if (Object->IsAsset() && GIsEditor)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(Object);
			GEditor->GetSelectedObjects()->Deselect(Object);
		}
	}

	// Broadcast the eminent objects destruction (ex. tell BlueprintActionDatabase to release its reference(s) on Blueprint(s) right now)
	FEditorDelegates::OnAssetsPreDelete.Broadcast(ObjectsToPurge);

	// Mark objects as purgeable.
	for (UObject* Object : ObjectsToPurge)
	{
		if (Object->IsRooted())
		{
			Object->RemoveFromRoot();
		}
		Object->ClearFlags(RF_Public | RF_Standalone);
	}

	// TODO: Revisit force replacing reference, current implementation is too aggressive and causes instability
	// If we have any object that were made purgeable, null out their references so we can garbage collect
	//if (ObjectsToPurge.Num() > 0)
	//{
	//	ObjectTools::ForceReplaceReferences(nullptr, ObjectsToPurge);
	//
	//}

	// Check if the map being edited is going to be purged. (b/c it's being deleted)
	if (bEditedMapPurged)
	{
		// The world being edited was purged and cannot be saved anymore, even with 'Save Current As', replace it by something sensible.
		FString StartupMapPackage = GetDefault<UGameMapsSettings>()->EditorStartupMap.GetLongPackageName();
		if (FPackageName::DoesPackageExist(StartupMapPackage))
		{
			UEditorLoadingAndSavingUtils::NewMapFromTemplate(StartupMapPackage, /*bSaveExistingMap*/false); // Expected to run GC internally.
		}
		else
		{
			UEditorLoadingAndSavingUtils::NewBlankMap(/*bSaveExistingMap*/false); // Expected to run GC internally.
		}
	}
	// if we have object to purge but the map isn't one of them collect garbage (if we purged the map it has already been done)
	else if (ObjectsToPurge.Num() > 0)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
#endif // WITH_EDITOR

}

bool HotReloadPackages(const TConstArrayView<FName>& InPackageNames)
{
	if (InPackageNames.Num() == 0)
	{
		return true;
	}

#if WITH_EDITOR
	// Flush loading and clean-up any temporary placeholder packages (due to a package previously being missing on disk)
	FlushAsyncLoading();
	{
		bool bRunGC = false;
		for (const FName& PackageName : InPackageNames)
		{
			bRunGC |= FLinkerLoad::RemoveKnownMissingPackage(PackageName);
		}
		if (bRunGC)
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}

	// Find the packages in-memory to content hot-reload
	TArray<UPackage*> ExistingPackages;
	Algo::Transform(InPackageNames, ExistingPackages, [](FName PackageName)
	{
		return FindPackage(nullptr, *PackageName.ToString());
	});
	return HotReloadPackages(ExistingPackages);
#else
	return true;
#endif
}

bool HotReloadPackages(const TConstArrayView<UPackage*>& InPackages)
{
#if WITH_EDITOR
	bool bAddPersistentLevel = false;
	
	TArray<UPackage*> FilteredPackages;
	for (UPackage* Package : InPackages)
	{
		if (Package)
		{
			if (Package->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				Package->ClearPackageFlags(PKG_NewlyCreated);
			}
			if (Package->ContainsMap())
			{
				bAddPersistentLevel = ShouldReloadPersistentLevel(Package);
			}

			if (!Package->ContainsMap() || Package->IsDirty())
			{
				FilteredPackages.Add(Package);
			}
		}
	}

	UWorld* CurrentWorld = GetCurrentWorld();
	if (CurrentWorld && bAddPersistentLevel)
	{
		ULevel* PersistentLevel = CurrentWorld->PersistentLevel;
		if (PersistentLevel && !FilteredPackages.Contains(PersistentLevel->GetPackage()))
		{
			FilteredPackages.Add(PersistentLevel->GetPackage());
		}
	}

	if (FilteredPackages.Num() > 0)
	{
		FlushRenderingCommands();

		FText ErrorMessage;
		UPackageTools::ReloadPackages(FilteredPackages, ErrorMessage, UPackageTools::EReloadPackagesInteractionMode::AssumePositive);

		if (!ErrorMessage.IsEmpty())
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
			return false;
		}
	}
#endif
	return true;
}

TArray<UPackage*> GetDirtyPackages()
{
	TArray<UPackage*> DirtyPackages;
#if WITH_EDITOR
	UEditorLoadingAndSavingUtils::GetDirtyMapPackages(DirtyPackages);
	UEditorLoadingAndSavingUtils::GetDirtyContentPackages(DirtyPackages);
#endif
	return DirtyPackages;
}

void AppendExternalPersistentLevelForReload(
	TArray<FName>& InOutPackagesPendingHotReload,
	TConstArrayView<FName> InPackagesPendingPurge)
{
#if WITH_EDITOR
	UWorld* CurrentWorld = GetCurrentWorld();
	if (!CurrentWorld)
	{
		return;
	}

	ULevel* PersistentLevel = CurrentWorld->PersistentLevel;
	if (!PersistentLevel || !PersistentLevel->IsUsingExternalObjects())
	{
		return;
	}

	const FName PackageName = PersistentLevel->GetPackage()->GetFName();
	if (!InPackagesPendingPurge.Contains(PackageName)
		&& !InOutPackagesPendingHotReload.Contains(PackageName))
	{
		InOutPackagesPendingHotReload.Add(PackageName);
	}
#endif
}
}
