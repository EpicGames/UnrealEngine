// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxFileUtils.h"

#include "Data/ManifestData.h"
#include "JsonObjectConverter.h"
#include "LogFileSandbox.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Sandbox/Platform/SanboxedFilePathUtils.h"
#include "Sandbox/Platform/SandboxedPlatformFilePath.h"
#include "Types/EBreakBehavior.h"
#include "Utils/SandboxDirectoryUtils.h"

namespace UE::FileSandboxCore
{
bool SaveManifest(const FFileSandboxCore_ManifestData& InData, const FString& InRootDirectory)
{
	FString JSONPayload;
	FJsonObjectConverter::UStructToJsonObjectString(InData, JSONPayload, 0, 0);
	const FString AbsFileName = InRootDirectory / GetManifestFileName();
	const bool bSuccess = FFileHelper::SaveStringToFile(JSONPayload, *AbsFileName);
	UE_CLOGF(!bSuccess, LogFileSandbox, Error, "Failed to write sandbox manifest file %ls", *InRootDirectory);
	return bSuccess;
}

TOptional<FFileSandboxCore_ManifestData> LoadManifest(const FString& InRootDirectory, bool bLogErrorIfNotExist)
{
	FFileSandboxCore_ManifestData ManifestPlatformFileState;
	FString JSONPayload;
	
	const FString AbsFileName = InRootDirectory / GetManifestFileName();
	if (FFileHelper::LoadFileToString(JSONPayload, *AbsFileName))
	{
		const TOptional<FFileSandboxCore_ManifestData> Result = LoadManifestFromContent(JSONPayload);
		UE_CLOGF(!Result.IsSet(), LogFileSandbox, Error, "Failed to parse sandbox manifest file %ls", *InRootDirectory);
		return Result;
	}
	
	UE_CLOGF(bLogErrorIfNotExist, LogFileSandbox, Error, "Failed to load sandbox manifest file %ls", *InRootDirectory);
	return {};
}

TOptional<FFileSandboxCore_ManifestData> LoadManifestFromContent(const FString& InFileContent)
{
	FFileSandboxCore_ManifestData ManifestPlatformFileState;
	const FString& JSONPayload = InFileContent;
	const bool bSuccess = FJsonObjectConverter::JsonObjectStringToUStruct(JSONPayload, &ManifestPlatformFileState, 0, 0);
	return bSuccess ? ManifestPlatformFileState : TOptional<FFileSandboxCore_ManifestData>{};
}

FString GetManifestFileName()
{
	return TEXT("manifest.json");
}

FString GetMetadataFileName()
{
	return TEXT("metadata.json");
}

FString GetSandboxMountPointRoot(const FString& InSandboxRootPath)
{
	return InSandboxRootPath / TEXT("Sandbox");
}

void EnumerateMountPoints(TFunctionRef<EBreakBehavior(const FString& InAssetPath, const FString& InFilesystemPath)> InProcess)
{
	TArray<FString> RootPaths;
	FPackageName::QueryRootContentPaths(RootPaths);
	for (const FString& AssetPath : RootPaths)
	{
		const FString FilesystemPath = FPackageName::LongPackageNameToFilename(AssetPath);
		if (InProcess(AssetPath, FilesystemPath) == EBreakBehavior::Break)
		{
			break;
		}
	}	
}

TOptional<FDateTime> GetSandboxTimestamp(
	const FString& InNonSandboxPath, 
	const FString& InRootSandboxPath, const FFileSandboxCore_ManifestData& InManifest
	)
{
	const TArray<FString>& RemovedFiles = InManifest.DeletedFiles.GetFiles();
	const int32 RemovedIndex = InManifest.DeletedFiles.GetFiles().IndexOfByPredicate([&InNonSandboxPath](const FString& RemovedFilePath)
	{
		return FPaths::IsSamePath(InNonSandboxPath, RemovedFilePath);
	});
	if (RemovedFiles.IsValidIndex(RemovedIndex))
	{
		// The file may have been removed but the user could have tempered with the manifest such that the timestamps are not the same length.
		const TArray<FDateTime>& Timestamps = InManifest.DeletedFiles.GetTimestamps();
		return Timestamps.IsValidIndex(RemovedIndex) ? Timestamps[RemovedIndex] : FDateTime::MinValue();
	}
	
	FDateTime EditOrAddTime = FDateTime::MinValue();
	const auto ProcessMountPoint = [&InRootSandboxPath, &InNonSandboxPath, &EditOrAddTime](const FString& InAssetPath, const FString& InFilesystemPath)
		{
			const FSandboxedPlatformFilePath MountPoint = FSandboxedPlatformFilePath::CreateMountPoint(
				GetSandboxMountPointRoot(InRootSandboxPath), InAssetPath, InFilesystemPath
			);
			const TOptional<FSandboxedPlatformFilePath> FilePath = MakeSandboxedFilePathIfInMountPoint(InNonSandboxPath, MountPoint);
			if (FilePath && ensure(FilePath->HasSandboxPath()))
			{
				EditOrAddTime = IFileManager::Get().GetTimeStamp(*FilePath->GetSandboxPath());
				return EBreakBehavior::Break;
			}
			return EBreakBehavior::Continue;
		};
	EnumerateMountPoints(ProcessMountPoint);
	return EditOrAddTime;
}
}
