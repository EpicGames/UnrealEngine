// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/FunctionFwd.h"
#include "HAL/Platform.h"

class FString;
enum class EBreakBehavior : uint8;
struct FDateTime;
struct FFileSandboxCore_ManifestData;
struct FFileSandboxCore_SandboxMetaData;
template<typename OptionalType> struct TOptional;

namespace UE::FileSandboxCore
{
/** Saves the manifest file. */
bool SaveManifest(const FFileSandboxCore_ManifestData& InData, const FString& InRootDirectory);
/** Loads the manifest file. */
TOptional<FFileSandboxCore_ManifestData> LoadManifest(const FString& InRootDirectory, bool bLogErrorIfNotExist = true);
/** Loads the manifest if you already have the file content. */
TOptional<FFileSandboxCore_ManifestData> LoadManifestFromContent(const FString& InFileContent);

/** Reads metadata about a sandbox if you already have the content of the file. */
TOptional<FFileSandboxCore_SandboxMetaData> LoadMetaDataFromFileContent(const FString& InFileContent);

/* @return The name to give the manifest file. */
FString GetManifestFileName();
/* @return The name to give the metadata file. */
FString GetMetadataFileName();

/** @return Path to directory in the sandbox file structure where mount points are stored, i.e. "MySandbox/Sandbox" */
FString GetSandboxMountPointRoot(const FString& InSandboxRootPath);

/** Enumerates all mount points in the engine. */
void EnumerateMountPoints(TFunctionRef<EBreakBehavior(const FString& InAssetPath, const FString& InFilesystemPath)> InProcess);

/** @return Timestamp in local time when InNonSandboxPath was edited, added, or removed by the sandbox. */
TOptional<FDateTime> GetSandboxTimestamp(const FString& InNonSandboxPath, const FString& InRootSandboxPath, const FFileSandboxCore_ManifestData& InManifest);
}