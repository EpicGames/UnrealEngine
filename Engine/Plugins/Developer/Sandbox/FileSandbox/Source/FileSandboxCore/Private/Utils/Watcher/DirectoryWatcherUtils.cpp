// Copyright Epic Games, Inc. All Rights Reserved.

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER

#include "DirectoryWatcherUtils.h"

#include "DirectoryWatcherModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

namespace UE::FileSandboxCore
{
FDirectoryWatcherModule& GetDirectoryWatcherModule()
{
	const FName DirectoryWatcherModuleName = TEXT("DirectoryWatcher");
	return FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(DirectoryWatcherModuleName);
}

FDirectoryWatcherModule* GetDirectoryWatcherModuleIfLoaded()
{
	const FName DirectoryWatcherModuleName = TEXT("DirectoryWatcher");
	if (FModuleManager::Get().IsModuleLoaded(DirectoryWatcherModuleName))
	{
		return &FModuleManager::GetModuleChecked<FDirectoryWatcherModule>(DirectoryWatcherModuleName);
	}
	return nullptr;
}

IDirectoryWatcher* GetDirectoryWatcher()
{
	FDirectoryWatcherModule& DirectoryWatcherModule = GetDirectoryWatcherModule();
	return DirectoryWatcherModule.Get();
}

IDirectoryWatcher* GetDirectoryWatcherIfLoaded()
{
	if (FDirectoryWatcherModule* DirectoryWatcherModule = GetDirectoryWatcherModuleIfLoaded())
	{
		return DirectoryWatcherModule->Get();
	}
	return nullptr;
}
}
#endif