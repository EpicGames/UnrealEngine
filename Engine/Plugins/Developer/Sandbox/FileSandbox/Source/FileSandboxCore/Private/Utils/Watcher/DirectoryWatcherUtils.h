// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER

class IDirectoryWatcher;
class FDirectoryWatcherModule;

namespace UE::FileSandboxCore
{
FDirectoryWatcherModule& GetDirectoryWatcherModule();
FDirectoryWatcherModule* GetDirectoryWatcherModuleIfLoaded();
IDirectoryWatcher* GetDirectoryWatcher();
IDirectoryWatcher* GetDirectoryWatcherIfLoaded();
}

#endif