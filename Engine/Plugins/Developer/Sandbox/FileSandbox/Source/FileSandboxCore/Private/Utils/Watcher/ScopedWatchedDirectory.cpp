// Copyright Epic Games, Inc. All Rights Reserved.

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER

#include "ScopedWatchedDirectory.h"

#include "DirectoryWatcherUtils.h"
#include "IDirectoryWatcher.h"

namespace UE::FileSandboxCore
{
FScopedWatchedDirectory::FScopedWatchedDirectory(FString InDirectory, FDelegateHandle InHandle)
	: WatchedDirectory(InDirectory)
	, Handle(InHandle)
{}

FScopedWatchedDirectory::FScopedWatchedDirectory(FScopedWatchedDirectory&& Old)
	: FScopedWatchedDirectory(MoveTemp(Old.WatchedDirectory), Old.Handle)
{
	Old.Handle.Reset();
}

FScopedWatchedDirectory::~FScopedWatchedDirectory()
{
	if (!Handle.IsValid() || WatchedDirectory.IsEmpty())
	{
		return;
	}
	
	if (IDirectoryWatcher* Watcher = GetDirectoryWatcherIfLoaded())
	{
		Watcher->UnregisterDirectoryChangedCallback_Handle(WatchedDirectory, Handle);
	}
}
}

#endif