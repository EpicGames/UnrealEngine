// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER

#include "Containers/UnrealString.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxCore
{
/** Unregisters IDirectoryWatcher when it goes out of scope. */
class FScopedWatchedDirectory : public FNoncopyable
{
public:
	
	FScopedWatchedDirectory() = default;
	FScopedWatchedDirectory(FString InDirectory, FDelegateHandle InHandle);
	FScopedWatchedDirectory(FScopedWatchedDirectory&& Old);
	~FScopedWatchedDirectory();
	
	const FString& GetWatchedDirectory() const { return WatchedDirectory; }
	bool IsValid() const { return Handle.IsValid(); }

private:
	
	FString WatchedDirectory;
	FDelegateHandle Handle;
};
}

#endif