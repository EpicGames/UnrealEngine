// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/ZipTransferUtils.h"

#include "Algo/AllOf.h"
#include "Data/ManifestData.h"
#include "FileUtilities/ZipArchiveReader.h"
#include "FileUtilities/ZipArchiveWriter.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "LogFileSandbox.h"
#include "Logging/LogMacros.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "MountPointUtils.h"
#include "SandboxFileUtils.h"
#include "FileChange/EFileChangeAction.h"
#include "FileUtilities/ZipFileMetaData.h"
#include "Utils/SandboxDirectoryUtils.h"

#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
#include "DirectoryWatcherModule.h"
#include "Watcher/DirectoryWatcherUtils.h"
#endif

namespace UE::FileSandboxCore::ZipUtils
{
namespace ZipDetail
{
static FDateTime ToLocalTime(const FDateTime& UtcTime)
{
	const FDateTime UtcNow = FDateTime::UtcNow();
	const FDateTime Now = FDateTime::Now();
	const FTimespan TimeZoneOffset = Now - UtcNow;
	return UtcTime + TimeZoneOffset;
}

static bool WriteToZip(const FString& InSandboxRoot, IFileHandle& InZipFile, TConstArrayView<FString> InFilesToCompress)
{
	IFileManager& FileManager = IFileManager::Get();
	// FZipArchiveWriter takes ownership of InZipFile. It handles releasing the resources.
	FZipArchiveWriter ZipArchiveWriter(&InZipFile, EZipArchiveOptions::RemoveDuplicate | EZipArchiveOptions::Deflate);
	
	for (const FString& FileToCompress : InFilesToCompress)
	{
		const TUniquePtr<FArchive> Reader = TUniquePtr<FArchive>(FileManager.CreateFileReader(*FileToCompress));
		if (!Reader)
		{
			UE_LOGF(LogFileSandbox, Error, "Failed to open %ls for reading", *FileToCompress)
			return false;
		}
		
		const int64 Size = Reader->TotalSize();
		TArray64<uint8> RawData;
		RawData.SetNumUninitialized(Size);
		Reader->Serialize(RawData.GetData(), Size);
		Reader->Close();
		
		// FileToCompress is an absolute path, e.g. "D:/MyProject/Intermediate/Sandboxes/MySandbox/manifest.json".
		// This converts it to "MySandbox/manifest.json".
		FString RelativeToRoot = FileToCompress;
		FPaths::MakePathRelativeTo(RelativeToRoot, *InSandboxRoot);
		
		// The file in the archive needs to have the same timestamp as in the file system so the edit time is correct when importing
		// File system returns time in UTC...
		FDateTime Timestamp = ToLocalTime(FileManager.GetTimeStamp(*FileToCompress)); 
		const bool bHasTimestamp = Timestamp != FDateTime::MinValue(); 
		const FDateTime TimestampToSet = bHasTimestamp ? Timestamp : FDateTime::Now();
		
		// ... but ZIP API expects time in local time
		ZipArchiveWriter.AddFile(RelativeToRoot, RawData, TimestampToSet);
	}
	
	return true;
}
}

bool ExportSandboxToZip(const FString& InSandboxRoot, const FString& InOutputZipPath)
{
	if (!ensure(FileSandboxCore::IsRootSandboxDirectory(InSandboxRoot)))
	{
		return false;
	}
	
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	
	// Find all the files in the directory
	TArray<FString> FilesToCompress;
	PlatformFile.FindFilesRecursively(FilesToCompress, *InSandboxRoot, TEXT(""));
	if (FilesToCompress.IsEmpty())
	{
		return false;
	}

	// Make sure that the output is .zip, in case the caller forgot it (e.g. they passed "D:/MyFoo/ZipFile" instead of "D:/MyFoo/ZipFile.zip"
	const FString CleansedOutput = FPaths::SetExtension(InOutputZipPath, TEXT("zip"));
	
	// Creates a zip file
	const bool bZipExisted = PlatformFile.FileExists(*CleansedOutput);
	IFileHandle* ZipFile = PlatformFile.OpenWrite(*CleansedOutput); 
	if (!ZipFile)
	{
		return false;
	}
	
	// WriteToZip takes ownership of the IFileHandle handle.
	if (!ZipDetail::WriteToZip(InSandboxRoot, *ZipFile, FilesToCompress))
	{
		// Unfortunately, FZipArchiveWriter API does not provide any API for cancelling. ~FZipArchiveWriter will just write, so we have to undo. 
		// It could be that before the operation, CleansedOutput already contained a file which is being overwritten.
		// In that case, this DeleteFile is an unfortunate casualty, but we'll assume that the user has already confirmed overwriting.
		UE_CLOGF(bZipExisted, LogFileSandbox, Warning, 
			"Failed to export sandbox. The Zip %ls already existed before. Clearing invalid data.", *CleansedOutput
			)
		PlatformFile.DeleteFile(*CleansedOutput);
		return false;
	}
	
	return true;
}

namespace ImportDetail
{
/** 
 * @return First directory in the path, e.g. "Foo" in "/Foo/Bar/", "Foo/Bar", or "/Foo/Bar.temp". 
 * Returns empty if no directory is contained, e.g. "/Foo". 
 */
static TOptional<FString> GetFirstDirectory(const FString& InPath)
{
	FString Normalized = InPath;
	FPaths::NormalizeFilename(Normalized);

	// Skip leading slash
	if (Normalized.StartsWith(TEXT("/")))
	{
		Normalized = Normalized.RightChop(1);
	}

	int32 SlashIndex;
	if (Normalized.FindChar(TEXT('/'), SlashIndex))
	{
		return Normalized.Left(SlashIndex);
	}

	return Normalized;
}

/** @return The path without the first directory, e.g. "Bar.txt" in "/Foo/Bar.txt". */
static TOptional<FString> RemoveFirstDirectory(const FString& InPath)
{
	FString Normalized = InPath;
	FPaths::NormalizeFilename(Normalized);

	// Skip leading slash
	if (Normalized.StartsWith(TEXT("/")))
	{
		Normalized = Normalized.RightChop(1);
	}

	int32 SlashIndex;
	if (Normalized.FindChar(TEXT('/'), SlashIndex))
	{
		// E.g. "Foo/MyText.text" -> SlashIndex = 3, Len = 15,
		// 15 - 3 = 12 -> Right(12) -> "/MyText.text"
		// 15 - 3 - 1 = 11 -> Right(11) -> "MyText.text"
		return Normalized.Right(Normalized.Len() - SlashIndex - 1);
	}

	// There is no first directory.
	return {};
}

static TOptional<FString> GetDirectorySharedByAllFiles(const TArray<FString>& InAllFiles)
{
	const TOptional<FString> SandboxName = InAllFiles.IsEmpty() ? TOptional<FString>() : ImportDetail::GetFirstDirectory(InAllFiles[0]);
	if (!SandboxName)
	{
		return SandboxName;
	}
	
	const bool bAllFilesInSameDirectory = Algo::AllOf(InAllFiles, [&SandboxName](const FString& InFile)
	{
		const TOptional<FString> FileFirstDirectory = ImportDetail::GetFirstDirectory(InFile);
		return FileFirstDirectory && *FileFirstDirectory == *SandboxName;
	});
	return bAllFilesInSameDirectory ? SandboxName : TOptional<FString>();
}

static bool CanParseRequiredFiles(const FString& InSandboxName, FImportInspectionResult& InOutResult)
{
	FZipArchiveReader& FileReader = *InOutResult.Reader;

	const FString ManifestFilePath = FPaths::Combine(*InSandboxName, FileSandboxCore::GetManifestFileName());
	const FString MetadataFilePath = FPaths::Combine(*InSandboxName, FileSandboxCore::GetMetadataFileName());
	
	TArray<uint8>& ManifestContentBytes = InOutResult.ManifestBytes;
	TArray<uint8>& MetadataContentBytes = InOutResult.MetadataBytes;
	const bool bHasManifest = FileReader.TryReadFile(ManifestFilePath, ManifestContentBytes);
	const bool bHasMetadata = FileReader.TryReadFile(MetadataFilePath, MetadataContentBytes);
	if (!bHasManifest || !bHasMetadata)
	{
		return false;
	}
	
	FString ManifestContent, MetadataContent;
	FFileHelper::BufferToString(ManifestContent, ManifestContentBytes.GetData(), ManifestContentBytes.Num());
	FFileHelper::BufferToString(MetadataContent, MetadataContentBytes.GetData(), MetadataContentBytes.Num());
	
	InOutResult.MetaData = LoadMetaDataFromFileContent(MetadataContent);
	const bool bCanParseRequiredFiles = InOutResult.MetaData.IsSet() && LoadManifestFromContent(ManifestContent).IsSet();
	return bCanParseRequiredFiles;
}
}

FImportInspectionResult::FImportInspectionResult(TUniquePtr<FZipArchiveReader> InReader)
	: Reader(MoveTemp(InReader))
	, AllFileNames(Reader->GetFileNames())
{}

TOptional<FImportInspectionResult> InspectFileForImport(const FString& InPathToZip)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	IFileHandle* ZipFile = PlatformFile.OpenRead(*InPathToZip); 
	if (!ZipFile)
	{
		return {};
	}
	
	FImportInspectionResult Result(MakeUnique<FZipArchiveReader>(ZipFile));
	Result.AllFileNames = Result.Reader->GetFileNames();
	
	const TOptional<FString> SandboxName = ImportDetail::GetDirectorySharedByAllFiles(Result.AllFileNames);
	if (!SandboxName || !ImportDetail::CanParseRequiredFiles(*SandboxName, Result))
	{
		return Result;
	}
	
	// Setting SandboxName indicates that the sandbox is valid.
	Result.SandboxName = *SandboxName;
	return Result;
}


namespace ImportDetail
{
static bool ImportZippedFile(const TArray<uint8>& InFileContent, const FString& InTargetFile, const TOptional<FDateTime>& InTimestamp)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString Directory = FPaths::GetPath(InTargetFile);
	if (!PlatformFile.CreateDirectoryTree(*Directory))
	{
		return false;
	}
	
	IFileManager& FileManager = IFileManager::Get();
	const TUniquePtr<FArchive> Archive(FileManager.CreateFileWriter(*InTargetFile, 0));
	if (!Archive)
	{
		return false;
	}
	
	Archive->Serialize(const_cast<uint8*>(InFileContent.GetData()), InFileContent.Num());
	Archive->Close(); // Always explicitly close to catch errors from flush/close
	const bool bIsSuccess = !Archive->IsError() && !Archive->IsCriticalError();
	
	// The timestamp of created file should be equal to timestamp the file has in the zip archive. 
	if (bIsSuccess && InTimestamp)
	{
		FileManager.SetTimeStamp(*InTargetFile, *InTimestamp);
	}
	
	return bIsSuccess;
}

static void DeleteFiles(const TArray<FString>& InFiles)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	for (const FString& File : InFiles)
	{
		PlatformFile.DeleteFile(*File);
	}
}

static void NotifyDirectoryWatcher(const TArray<FString>& InFiles)
{
#if UE_SANDBOX_WITH_DIRECTORY_WATCHER
	// Notify that the sandboxed directories have been restored to their original state
	FDirectoryWatcherModule* DirectoryWatcherModule = GetDirectoryWatcherModuleIfLoaded();
	if (!DirectoryWatcherModule)
	{
		return;
	}
	
	TArray<FFileChangeData> FileChanges;
	Algo::Transform(InFiles, FileChanges, [](const FString& InFile)
	{
		return FFileChangeData(InFile, FFileChangeData::FCA_Added);
	});
	DirectoryWatcherModule->RegisterExternalChanges(FileChanges);
#endif
}

template<typename TGetCachedFileContent> requires std::is_invocable_r_v<const TArray<uint8>*, TGetCachedFileContent, const FString& /*InFileName*/>
static bool ImportFromZip(
	const FZipArchiveReader& InReader, const TArray<FString>& InFilesInZipArchive, const FString& InSandboxTargetRoot, 
	TGetCachedFileContent&& InGetCachedFileContent
	)
{
	// 1. Nothing can be in the directory that we'll import to.
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.DeleteDirectoryRecursively(*InSandboxTargetRoot);
	
	// 2. Copy all the files from the zip
	TArray<FString> FilesImportedSoFar;
	TArray<uint8> FileContent;
	bool bFreeOfErrors = true;
	for (int32 Index = 0; Index < InFilesInZipArchive.Num() && bFreeOfErrors; ++Index)
	{
		const FString& ZippedFile = InFilesInZipArchive[Index];
		
		// All paths are "SandboxName/Some/Foo.bar" but we want only want "Some/Foo.bar" to combine it with InSandboxTargetRoot.
		const TOptional<FString> PathInSandbox = RemoveFirstDirectory(ZippedFile);
		if (!ensure(PathInSandbox))
		{
			bFreeOfErrors = false;
			break;
		}
		
		// E.g. InSandboxTargetRoot = "D:Project/Intermediate/Sandboxes/", PathInSandbox = "Sandbox/Game/Material.uasset".
		const FString Destination = FPaths::Combine(InSandboxTargetRoot, *PathInSandbox);
		// This is for security to avoid "Zip Slips": the zip could contain "../../Outside.txt", so it would write outside of the intended directory.
		// Code calling ImportFromZip should have already validated this, but we'll do it once more here to make sure.
		if (!ensure(FPaths::IsUnderDirectory(Destination, InSandboxTargetRoot)))
		{
			bFreeOfErrors = false;
			break;
		}
		
		// Optimization: Is it a file that we already deserialized during inspection?
		if (const TArray<uint8>* CachedContent = InGetCachedFileContent(ZippedFile))
		{
			const TOptional<FDateTime> NoTimestamp; // We don't care for timestamps on the manifest or metadata file
			bFreeOfErrors &= ImportZippedFile(*CachedContent, Destination, NoTimestamp);
			FilesImportedSoFar.Add(Destination);
			continue;
		}
		
		FileContent.Empty(FileContent.Num());
		FileUtilities::FZipFileMetaData Metadata;
		if (InReader.TryReadFile(ZippedFile, FileContent, nullptr, &Metadata))
		{
			bFreeOfErrors &= ImportZippedFile(FileContent, Destination, Metadata.Timestamp);
			FilesImportedSoFar.Add(Destination);
		}
		else
		{
			bFreeOfErrors = false;
		}
	}
	
	// 3. Potentially rollback any changes we've made if there were errors.
	if (bFreeOfErrors)
	{
		// This is needed so FWatchedSandboxRepository is notified.
		NotifyDirectoryWatcher(FilesImportedSoFar);
	}
	else
	{
		DeleteFiles(FilesImportedSoFar);
	}
	
	return bFreeOfErrors;
}
}

bool ImportSandboxFromZip(const FImportInspectionResult& InInspection, const FString& InSandboxTargetRoot)
{
	const FString ManifestFilePath = FPaths::Combine(InInspection.SandboxName, FileSandboxCore::GetManifestFileName());
	const FString MetadataFilePath = FPaths::Combine(InInspection.SandboxName, FileSandboxCore::GetMetadataFileName());
	const bool bImported = ImportDetail::ImportFromZip(*InInspection.Reader, InInspection.AllFileNames, InSandboxTargetRoot,
		[&InInspection, &ManifestFilePath, &MetadataFilePath](const FString& InFileName) -> const TArray<uint8>*
		{
			if (InFileName == ManifestFilePath)
			{
				return &InInspection.ManifestBytes;
			}
			if (InFileName == MetadataFilePath)
			{
				return &InInspection.MetadataBytes;
			}
			return nullptr;
		});

	if (!bImported)
	{
		return false;
	}

	// The zip may have been produced on another machine, or with the .uproject in a different
	// location, so the file paths and mount points recorded in the manifest do not necessarily
	// resolve here. Rewrite them against the engine's currently-registered mount points and
	// persist the result back to disk.
	TOptional<FFileSandboxCore_ManifestData> Manifest = FileSandboxCore::LoadManifest(InSandboxTargetRoot);
	if (!Manifest.IsSet())
	{
		return false;
	}

	FileSandboxCore::MigrateManifestFilePaths(*Manifest);
	return FileSandboxCore::SaveManifest(*Manifest, InSandboxTargetRoot);
}
}
