// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"
#include "UObject/SoftObjectPath.h"
#include "InstancedStructMap.generated.h"

/** Advanced tagging structure. Each USTRUCT() can be added at most once. */
USTRUCT()
struct FFileSandboxCore_InstancedStructMap
{
	GENERATED_BODY()

	/** @return The existing struct data, or newly created data. */
	FStructView FindOrAddData(UScriptStruct* InStruct);

	/** @return The existing struct data, or an empty view if not present. */
	FStructView FindData(UScriptStruct* InStruct);
	
	/** Removes a struct entry. */
	void Remove(UScriptStruct* InStruct);

	/** @return The existing struct data, or newly created data. */
	template<typename T>
	T& FindOrAddData();

	/** @return The existing struct data, or an empty view if not present. */
	template<typename T> 
	T* FindData();

	/** Removes a struct entry. */
	template<typename T> 
	void Remove();

private:

	UPROPERTY()
	TMap<FSoftObjectPath, FInstancedStruct> Data;
};

inline FStructView FFileSandboxCore_InstancedStructMap::FindOrAddData(UScriptStruct* InStruct)
{
	FInstancedStruct& StructData = Data.FindOrAdd(InStruct);
	if (StructData.GetScriptStruct() == InStruct)
	{
		return StructData;
	}

	StructData.InitializeAs(InStruct);
	return StructData;
}

inline FStructView FFileSandboxCore_InstancedStructMap::FindData(UScriptStruct* InStruct)
{
	FInstancedStruct* Struct = Data.Find(InStruct);
	return Struct ? FStructView(*Struct) : FStructView();
}

inline void FFileSandboxCore_InstancedStructMap::Remove(UScriptStruct* InStruct)
{
	Data.Remove(InStruct);
}

template <typename T>
T& FFileSandboxCore_InstancedStructMap::FindOrAddData()
{
	return FindOrAddData(T::StaticStruct()).template Get<T>();
}

template <typename T>
T* FFileSandboxCore_InstancedStructMap::FindData()
{
	FStructView Result = FindData(T::StaticStruct());
	return Result.GetPtr<T>();
}

template <typename T>
void FFileSandboxCore_InstancedStructMap::Remove()
{
	Remove(T::StaticStruct());
}
