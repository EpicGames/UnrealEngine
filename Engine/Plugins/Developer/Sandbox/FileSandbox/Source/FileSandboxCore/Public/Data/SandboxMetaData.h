// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "InstancedStructMap.h"

#include "SandboxMetaData.generated.h"

/** Metadata about a sandbox. This data is not needed for operations of the sandbox itself and is intended to be useful for external systems. */
USTRUCT()
struct FFileSandboxCore_SandboxMetaData
{
	GENERATED_BODY()

	/** Name of this sandbox, usually provided by the user. */
	UPROPERTY()
	FString Name;

	/** Description of this sandbox, usually provided by the user. */
	UPROPERTY()
	FString Description;

	/** Custom tags that may have been added to this sandbox. */
	UPROPERTY()
	TArray<FName> Tags;

	/** Custom advanced data that may have been added to this sandbox. */
	UPROPERTY()
	FFileSandboxCore_InstancedStructMap CustomData;

	FFileSandboxCore_SandboxMetaData() = default;
	explicit FFileSandboxCore_SandboxMetaData(FString InName, FString InDescription = {})
		: Name(MoveTemp(InName))
		, Description(MoveTemp(InDescription))
	{}
};