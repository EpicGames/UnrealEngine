// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VersionInfo.generated.h"

class FEngineVersion;
struct FCustomVersion;

/** Holds file version information */
USTRUCT()
struct FFileSandboxCore_FileVersionInfo
{
	GENERATED_BODY()

	/* UE5 File version */
	UPROPERTY()
	int32 FileVersionUE5 = 0;
	
	/* Licensee file version */
	UPROPERTY()
	int32 FileVersionLicensee = 0;

	/** Initialize this version info from the compiled in data */
	FILESANDBOXCORE_API void Initialize();
	/** @return File version info as string */
	FILESANDBOXCORE_API FString ToString() const;
};

/** Holds engine version information */
USTRUCT()
struct FFileSandboxCore_EngineVersionInfo
{
	GENERATED_BODY()

	/** Major version number */
	UPROPERTY()
	uint16 Major = 0;

	/** Minor version number */
	UPROPERTY()
	uint16 Minor = 0;

	/** Patch version number */
	UPROPERTY()
	uint16 Patch = 0;

	/** Changelist number. This is used to arbitrate when Major/Minor/Patch version numbers match */
	UPROPERTY()
	uint32 Changelist = 0;

	/** Initialize this version info from the given version */
	FILESANDBOXCORE_API void Initialize(const FEngineVersion& InVersion);

	/** @return Engine version info as string, e.g. "5.7.1-48893048" */
	FILESANDBOXCORE_API FString ToString(bool bIncludeChangelist = true) const;

	/**
	 * Check if this version is compatible with the given version.
	 * A sandbox version is compatible if it's not newer than the current version.
	 * Comparison order: Major > Minor > Patch > Changelist
	 * @param InCurrentVersion The current engine version to check against
	 * @return true if this version can be loaded in the current version
	 */
	FILESANDBOXCORE_API bool IsCompatibleWith(const FFileSandboxCore_EngineVersionInfo& InCurrentVersion) const;
};

/** Holds custom version information */
USTRUCT()
struct FFileSandboxCore_CustomVersionInfo
{
	GENERATED_BODY()

	friend FArchive& operator<<(FArchive& Archive, FFileSandboxCore_CustomVersionInfo& VersionInfo)
	{
		Archive << VersionInfo.FriendlyName;
		Archive << VersionInfo.Key;
		Archive << VersionInfo.Version;
		return Archive;
	}
	
	/** Friendly name of the version */
	UPROPERTY()
	FName FriendlyName;

	/** Unique custom key */
	UPROPERTY()
	FGuid Key;

	/** Custom version */
	UPROPERTY()
	int32 Version = 0;

	/** Initialize this version info from the given version */
	FILESANDBOXCORE_API void Initialize(const FCustomVersion& InVersion);
};

/** Holds version information for a session */
USTRUCT()
struct FFileSandboxCore_VersionInfo
{
	GENERATED_BODY()
	
	/** File version info */
	UPROPERTY()
	FFileSandboxCore_FileVersionInfo FileVersion;

	/** Engine version info */
	UPROPERTY()
	FFileSandboxCore_EngineVersionInfo EngineVersion;

	/** Custom version info */
	UPROPERTY()
	TArray<FFileSandboxCore_CustomVersionInfo> CustomVersions;

	/** Initialize this version info from the compiled in data */
	FILESANDBOXCORE_API void Initialize();
	/** @return Whether any data is set. */
	FILESANDBOXCORE_API bool IsInitialized() const;
	
	/** @eturn File and version info as string. Does not include custom versions. */
	FILESANDBOXCORE_API FString ToString() const;

};
