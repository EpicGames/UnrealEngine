// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IFileSandboxCoreModule.h"
#include "ISandboxManager.h"
#include "ISandboxInstance.h"

#include "Types/ArgsFwds.h"
#include "Types/Manager/NewSandboxArgs.h"
#include "Types/Manager/LoadSandboxByNameArgs.h"
#include "Types/Manager/DeleteSandboxResult.h"
#include "Types/Manager/DeleteSandboxByNameArgs.h"
