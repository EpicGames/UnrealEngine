// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::FileSandboxCore
{
class ISandboxInstance;
class ISandboxManager;
class ISandboxRepository;

class IFileSandboxCoreModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IFileSandboxCoreModule& Get() { return FModuleManager::LoadModuleChecked<IFileSandboxCoreModule>("FileSandboxCore"); }

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable() { return FModuleManager::Get().IsModuleLoaded("FileSandboxCore"); }
	
	/** @return The API for managing live sandboxes. */
	virtual ISandboxManager& GetSandboxManager() = 0;
	
	/** @return Gets the repository for the default sandbox save location (@see GetBaseSandboxDirectory) */
	virtual ISandboxRepository& GetDefaultSandboxRepository() = 0; 
	
	// In the future, we may consider adding CreateSandboxRepository(FString BasePath), or RegisterCustomSandboxRepository, to handle custom locations
};
}
