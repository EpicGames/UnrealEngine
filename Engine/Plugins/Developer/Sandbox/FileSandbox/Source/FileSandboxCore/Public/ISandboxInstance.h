// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/GatheredFileChanges.h"
#include "Types/Sandbox/RevertArgs.h"
#include "Types/Sandbox/PersistResult.h"
#include "Types/ArgsFwds.h"
#include "Types/SandboxedFileChangeInfo.h"

#define UE_API FILESANDBOXCORE_API

struct FFileSandboxCore_SandboxMetaData;

namespace UE::FileSandboxCore
{
enum class EFileChangeGatherFlags : uint8;
enum class EBreakBehavior : uint8;
struct FRevertResult;
struct FPersistArgs;
struct FPersistedFileInfo;

using FConsumeFileSignature = EBreakBehavior(const FString&);
	
/**
 * Manages the file sandbox.
 *
 * While this is alive, some engine I/O operations are sandboxed.
 * Only files in mount points (game content, engine content, plugin content) are tracked.
 * Other files, such as those residing in Saved/ or Config/ are not affected.
 */
class ISandboxInstance
{
public:
	
	/** Persists all files. */
	UE_API bool PersistAll();
	
	/**
	 * More advanced version of PersistAll that gives you more control how persisting should occur.
	 * @param InArgs	Parameters controlling the persist options specified by the user.
	 * @return Returns a persist result object that contains result status.
	 */
	virtual FPersistResult PersistSandbox(const FPersistArgs& InArgs) = 0;

	
	/** Reverts all files to the state they had before the sandbox. */
	virtual FRevertResult RevertAll() = 0;
	
	/** Reverts the files you specify to the state they had before the sandbox. */
	UE_EXPERIMENTAL(5.8, "Reverting of single files does not yet work for all cases. See UE-368478.")
	virtual FRevertResult RevertSpecified(const TConstArrayView<FString>& InFiles) = 0;
	
	/**
	 * More advanced version that gives you full control: specifically when packages are hot-reloaded and purged.
	 * This gathers the packages that you must hot-reload or purge from memory.
	 * @param InArgs Input for the revert operation
	 * @return Information about the revert operation; you must handle hot-reloading and purging yourself.
	 * @see HotReloadPackages, PurgePackages
	 */
	UE_EXPERIMENTAL(5.8, "Reverting of single files does not yet work for all cases. See UE-368478.")
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	[[nodiscard]] virtual FRevertResult RevertSandbox(const FRevertArgs& InArgs = {}) = 0;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	
	
	/** Enumerates all files that have been changed (added, removed, or edited). */
	UE_API FGatheredFileChanges GatherChangedFiles(EFileChangeGatherFlags InFlags = EFileChangeGatherFlags::All) const;
	
	/** 
	 * Invokes InProcess for every effective file change that would be performed if the sandbox was persisted.
	 * @see ISandboxManager::EnumerateFileChanges.
	 */
	virtual void EnumerateFileChanges(
		TFunctionRef<FProcessFileChangeSignature> InProcess, EFileEnumerationFlags InFlags = EFileEnumerationFlags::All
		) const = 0;
	
	/** 
	 * @return Whether any changes have to any files.
	 * @note This does not include packages with in-memory changes only.
	 * @see GetDirtyPackages
	 */
	UE_API bool HasFileChanges() const;
	
	/** @return The timestamp at which the file was last modified by the sandbox, or FDateTime::MinValue(). */
	virtual TOptional<FDateTime> GetSandboxedFileTimestamp(const FString& InFilePath) const = 0;

	
	/** @return Whether the given package file exists on non sandbox path, i.e. whether the file was deleted in sandbox mode. */
	virtual bool DeletedPackageExistsInNonSandbox(const FString& InFilename) const = 0;
	
	/**
	 * @return Metadata about this sandbox.
	 * 
	 * @note This is the metadata that the sandbox has when it was created / loaded.
	 * If the underlying file has changed, this will not return the latest value.
	 */
	virtual const FFileSandboxCore_SandboxMetaData& GetInitialMetaData() const = 0;

	/** @return The root directory of the sandbox. */
	virtual const TCHAR* GetRootDirectory() const = 0;
	
	/** @return Delegate invoked when the result of GatherChangedFiles has changed. */
	virtual FSimpleMulticastDelegate& OnSandboxedFilesChanged() = 0;

	virtual ~ISandboxInstance() = default;
};
}

#undef UE_API
