// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Types/ArgsFwds.h"
#include "Types/GatheredFileChanges.h"
#include "Types/Manager/LeaveSandboxArgs.h"
#include "Types/Manager/LeaveSandboxResult.h"
#include "Types/Manager/SandboxCreationResult.h"
#include "Types/SandboxedFileChangeInfo.h"
#include "Utils/SandboxDirectoryUtils.h"

#define UE_API FILESANDBOXCORE_API

namespace UE::FileSandboxCore
{
class ISandboxInstance;
class ISandboxLock;
enum class EBreakBehavior : uint8;
enum class ESandboxFileChange : uint8;
	
DECLARE_MULTICAST_DELEGATE_OneParam(FSandboxInstanceEvent, ISandboxInstance&);
using FSandboxShutdownEvent = FSimpleMulticastDelegate;
	
/**
 * Manages the lifetime of the ISandboxInstance.
 * For now, only one sandbox can be active at any given time and all files in mount points are sandboxed.
 *
 * ===== Overview =====
 * During sandbox, any I/O operations that modify, delete, or add files in mount points (e.g. /Game/..., /Engine/..., /MyPlugin/..., etc.) will
 * be sandboxed. That means those operations are performed in a separate sandbox file system without changing the underlying files. At any time,
 * you can persist changes, i.e. apply them to the underlying file system where the real files reside.
 *
 * ==== Example =====
 * 1. The user opens the asset "Game/Levels/MyBlueprint.uasset", changes it, and then clicks save.
 * 2. The contained UObjects are serialized and a low-level I/O operation is triggered (via IPlatformFile).
 *	2.1 Normally, the data would be written "D:/MyProject/Content/Levels/MyBlueprint.uasset",
 *	2.2 But we redirect the data to a sandboxed location, e.g."D:/MyProject/Intermediate/Sandboxes/YourSandbox/Sandbox/Game".
 * Similar behaviour goes for when a file is deleted or added.
 *	
 * ==== Persisting =====
 * Persisting effectively means to apply the changes to the original files.
 * Files marked for delete are deleted from the original location.
 * Added and modified files are copied from the sandbox location to the original location, e.g. a modified file
 * "D:/MyProject/Content/Levels/MyBlueprint.uasset" would be replaced with "D:/MyProject/Intermediate/Sandboxes/YourSandbox/Sandbox/Game".
 *
 * ===== Sandbox file structure =====
 * By default, sandboxes are saved to the Intermediate/Sandboxes directory (@see GetBaseSandboxDirectory).
 * Sandboxes are stored in separate directories, e.g. a sandbox named "MySandbox" would be saved in "MyProject/Intermediate/Sandboxes/MySandbox".
 * 
 * The directory contains a "Manifest.json" file at the root level; it contains bookkeeping data (which files were added, modified, deleted) and
 * metadata, such as user given description and custom metadata specific to the business logic.
 * So for example, there would be a file "MyProject/Intermediate/Sandboxes/MySandbox/Manifest.json".
 * 
 * Furthermore, changed and added files are saved in the sub-directory "Sandbox". This directory contains one directory name per mount point,
 * like "Game", "Engine", "MyPlugin". Within those mount point directories files are added according to the folder structure of the package name.
 * So for example:
 * - "MyProject/Intermediate/Sandboxes/MySandbox/Sandboxes/Game/Levels/MyLevel.uasset" maps to "Game/Levels/MyLevel.uasset" (mount = "/Game/"
 * - "MyProject/Intermediate/Sandboxes/MySandbox/Sandboxes/MyPlugin/FooBlueprint.uasset" maps to "MyPlugin/FooBlueprint.uasset" (mount = "/MyPlugin/")
 */
class ISandboxManager
{
public:
	
	/**
	 * Puts the engine into a sandbox: This creates a new environment.
	 * 
	 * Tip: If you don't care about the specifics of potential errors, you can write if (ISandboxInstance* Instance = Manager->CreateNewSandbox(...)).
	 * @note Do not keep any raw reference to ISandboxInstance as it turns stale when the sandbox is shutdown.
	 * @see FWeakSandboxPtr, GetGlobalSandboxInstance.
	 */
	virtual FNewSandboxResult CreateNewSandbox(const FNewSandboxArgs& InArgs) = 0;

	
	/**
	 * Puts the engine into a sandbox: applies the state from a pre-existing sandbox to any open assets.
	 * 
	 * Any assets that are currently open / in-memory will be hot-reloaded.
	 * Effectively, this means that their file content will be re-loaded but from the sandboxed version of the .uasset file.
	 * 
	 * Tip: If you don't care about the specifics of potential errors, you can write if (ISandboxInstance* Instance = Manager->LoadNamedSandbox(...)).
	 * @note Do not keep any raw reference to ISandboxInstance as it turns stale when the sandbox is shutdown.
	 * @see FWeakSandboxPtr, GetGlobalSandboxInstance.
	 */
	UE_API FLoadSandboxResult LoadNamedSandbox(const FLoadSandboxByNameArgs& InArgs);
	
	/** Advanced version of LoadNamedSandbox for when you know the specific root directory of a sandbox. */
	virtual FLoadSandboxResult LoadSandbox(const FLoadSandboxByDirectoryArgs& InLoadArgs) = 0;
	
	
	/** Deletes the sandbox with the given name, if found. It is valid to delete a sandbox if it is currently valid. */
	UE_API FDeleteSandboxResult DeleteNamedSandbox(const FDeleteSandboxByNameArgs& InArgs);
	
	/** Advanced version of DeleteNamedSandbox for when you know the specific root directory of a sandbox. */
	virtual FDeleteSandboxResult DeleteSandbox(const FDeleteSandboxByDirectoryArgs& InArgs) = 0;
	
	
	/** Leaves the current sandbox. */
	virtual FLeaveSandboxResult LeaveSandbox(const FLeaveSandboxArgs& InLeaveArgs = FLeaveSandboxArgs()) = 0;
	
	/**
	 * More advanced version of CanLeaveSandbox if you want to find out the reason why the sandbox cannot be left.
	 * @return Lock object that determines whether the sandbox can be left.
	 * @note Do not keep any raw reference to ISandboxLock as it turns stale when the sandbox is shutdown.
	 */
	virtual ISandboxLock* GetActiveLock() const = 0;
	
	/** @return Whether a LeaveSandbox call would be successful. */
	bool CanLeaveSandbox() const { return CanLeaveSandboxWithReason() == ELeaveSandboxErrorCode::Success; }
	/** @return Why a LeaveSandbox call would be (un-)successful. */
	UE_API ELeaveSandboxErrorCode CanLeaveSandboxWithReason() const;

	/**
	 * Gets the currently active global sandbox instance.
	 *
	 * The name “global” sandbox instance was chosen with the idea that, in the future, we may support directory bindings where only specific
	 * directories are sandboxed. For now (5.8), there can be at most one sandbox instance.
	 *
	 * @note Do not keep any raw reference to ISandboxInstance as it turns stale when the sandbox is shutdown.
	 * @see FWeakSandboxPtr, GetGlobalSandboxInstance.
	 */
	virtual ISandboxInstance* GetActiveSandboxInstance() const = 0;

	/** @return Whether the is any active sandbox. */
	bool HasActiveSandbox() const { return GetActiveSandboxInstance() != nullptr; }
	
	/** 
	 *  Invokes InProcess for every effective file change that would be performed if the sandbox was persisted.
	 * 
	 * Effective changes can differ from the actual actions performed during sandbox because external changes could have been made to the project
	 * outside of sandbox.
	 * 
	 * The implementation respects file changes that have occured since the sandbox the last time the sandbox was active.
	 * Examples: Suppose that while the engine is sandboxed file x is
	 * - removed: then, outside of sandbox, file x is added. Then X would not be listed (InProcess not invoked).
	 * - added: then, outside of sandbox, file x is added. Then X would be reported as edited.
	 * - edited: then, outside of sandbox, file x is removed. Then X would be reported as added.
	 * If no external changes are made, the file action is reported exactly as it was performed in the sandbox without changes.
	 */
	UE_API void EnumerateFileChangesByName(
		const FString& InSandboxName, TFunctionRef<FProcessFileChangeSignature> InProcess,
		const FString& InBaseDirectory = GetBaseSandboxDirectory()
		) const;
	
	/** Alternative version that uses the sandbox root directory instead.*/
	virtual void EnumerateFileChanges(
		const FString& InSandboxRootPath, TFunctionRef<FProcessFileChangeSignature> InProcess,
		EFileEnumerationFlags InFlags = EFileEnumerationFlags::All
		) const = 0;
	
	/** @return The timestamp at which the file was last modified by the sandbox. */
	virtual TOptional<FDateTime> GetSandboxedFileTimestamp(const FString& InSandboxRootPath, const FString& InFilePath) const = 0;
	
	/** @return Event invoked after a sandbox has been successfully created. */
	virtual FSandboxInstanceEvent& OnPostSandboxStartup() = 0;
	
	/** @return Event invoked when it has been decided, that a sandbox instance will be taken down, but before any core destruction logic runs. */
	virtual FSandboxInstanceEvent& OnPreSandboxShutdown() = 0;
	/** @return Event invoked at the end of shutting down a sandbox, i.e. after all core destruction logic has run. */
	virtual FSandboxShutdownEvent& OnPostSandboxShutdown() = 0;
	
	virtual ~ISandboxManager() = default;
};
}

#undef UE_API