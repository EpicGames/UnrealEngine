// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/VersionInfo.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/DateTime.h"

namespace UE::FileSandboxCore
{
enum class EBreakBehavior : uint8;
struct FRepositoryChangedEvent;
struct FSandboxMetaInfo;
	
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSandboxesChanged, const FRepositoryChangedEvent&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSandboxMetaDataChanged, const FString& /* InRootPath */);

/** 
 * Knows of the sandboxes instances available. 
 * 
 * You can think of this as a cache that will notify when the sandboxes available changes, and the metadata changes.
 * You should prefer this over manually loading metadata (@see LoadMetaData) whenever possible to avoid excessive file reads, JSON parsing, etc.
 * This is useful for, e.g. UI want to display the available sandboxes. 
 */
class ISandboxRepository
{
public:
	
	/** 
	 * Enumerates all known sandboxes. 
	 * InRootPath will be the absolute path to the root sandbox directory, i.e. the one containing the metadata file.
	 */
	virtual void ForEachSandbox(TFunctionRef<EBreakBehavior(const FString& InRootPath, const FSandboxMetaInfo& MetaData)> InProcessSandboxes) = 0;
	
	/** @return Number of distinct sandboxes, i.e. number of times ForEachSandbox would invoke the callback. */
	virtual int32 NumSandboxes() const = 0;
	
	
	/**
	 * @param InRootPath Relative or absolute path to the root directory of the sandbox, i.e. one enumerated by InProcessSandboxes. 
	 * @param InProcessMetadata Callback invoked if the metadata exists
	 * @return Whether the callback was invoked
	 */
	virtual bool ReadMetaData(const FString& InRootPath, TFunctionRef<void(const FSandboxMetaInfo&)> InProcessMetadata) = 0;
	
	
	/** @return Invoked when the known sandboxes changes (removed, discovered, etc.), on game thread. */
	virtual FOnSandboxesChanged& OnSandboxesChanged() = 0;
	
	/** @return Invoked when a sandbox's metadata changes, on game thread. */
	virtual FOnSandboxMetaDataChanged& OnSandboxMetaDataChanged() = 0;
	
	
	virtual ~ISandboxRepository() = default;
};
}
