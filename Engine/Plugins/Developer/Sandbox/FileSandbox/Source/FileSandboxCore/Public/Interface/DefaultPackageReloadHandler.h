// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPackageReloadHandler.h"

#define UE_API FILESANDBOXCORE_API 

namespace UE::FileSandboxCore
{
/** Default implementation for reloading assets. */
class FDefaultPackageReloadHandler : public IPackageReloadHandler
{
public:
	
	//~ Begin IPackageReloadHandler Interface
	UE_API virtual void PurgePackages(const FPurgePackageArgs& InArgs) override;
	UE_API virtual void HotReloadPackages(const FHotReloadPackageArgs& InArgs) override;
	//~ End IPackageReloadHandler Interface
};
}

#undef UE_API
