// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Algo/ForEach.h"
#include "Containers/Array.h"
#include "Interface/IPersistFeedback.h"

namespace UE::FileSandboxCore
{
/** Forwards the errors to other error handlers. */
class FAggregatePersistFeedback : public IPersistFeedback
{
public:
	
	explicit FAggregatePersistFeedback(TArray<IPersistFeedback*> InErrorHandlers) : ErrorHandlers(MoveTemp(InErrorHandlers)) {}
	explicit FAggregatePersistFeedback(IPersistFeedback& InErrorHandler) : ErrorHandlers({ &InErrorHandler }) {}
	
	//~ Begin IPersistFeedback Interface
	virtual void StartFile(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->StartFile(InFilename);
			}
		});
	}
	virtual void HandleSuccess(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleSuccess(InFilename);
			}
		});
	}
	virtual void HandleError_CheckoutNotAllowed(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleError_CheckoutNotAllowed(InFilename);
			}
		});
	}
	virtual void HandleError_Checkout(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleError_Checkout(InFilename);
			}
		});
	}
	virtual void HandleError_Revert(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleError_Revert(InFilename);
			}
		});
	}
	virtual void HandleError_MarkForAdd(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleError_MarkForAdd(InFilename);
			}
		});
	}
	virtual void HandleError_DeleteSCC(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleError_DeleteSCC(InFilename);
			}
		});
	}
	virtual void HandleError_MakeWritable(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleError_MakeWritable(InFilename);
			}
		});
	}
	virtual void HandleError_MoveFile(const TCHAR* InToFilename, const TCHAR* InFromFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleError_MoveFile(InToFilename, InFromFilename);
			}
		});
	}
	virtual void HandleError_DeleteFile(const TCHAR* InFilename) override
	{
		Algo::ForEach(ErrorHandlers, [&](IPersistFeedback* Handler)
		{
			if (Handler)
			{
				Handler->HandleError_DeleteFile(InFilename);
			}
		});
	}
	//~ End IPersistFeedback Interface
	
private:
	
	/** The error handlers to forward the errors to. */
	TArray<IPersistFeedback*> ErrorHandlers;
};
}
