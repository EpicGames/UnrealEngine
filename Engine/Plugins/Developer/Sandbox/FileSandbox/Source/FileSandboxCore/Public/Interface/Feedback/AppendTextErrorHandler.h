// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Interface/IPersistFeedback.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "FAppendTextPersistErrorHandler"

namespace UE::FileSandboxCore
{
class FAppendTextPersistFeedback : public IPersistFeedback
{
public:
	
	TArray<FText> AllErrors;
	
	//~ Begin IPersistFeedback Interface
	virtual void StartFile(const TCHAR* InFilename) override {}
	virtual void HandleSuccess(const TCHAR* InFilename) override {}
	virtual void HandleError_CheckoutNotAllowed(const TCHAR* InFilename) override
	{
		AllErrors.Emplace(FText::Format(
			LOCTEXT("CheckoutNotAllowed", "Checkout of file '{0}' from revision control was not allowed when persisting sandbox state!"),
			FText::AsCultureInvariant(InFilename)
			));
	}
	virtual void HandleError_Checkout(const TCHAR* InFilename) override
	{
		AllErrors.Emplace(FText::Format(
			LOCTEXT("Checkout", "Failed to checkout file '{0}' from revision control when persisting sandbox state!"),
			FText::AsCultureInvariant(InFilename)
			));
	}
	virtual void HandleError_Revert(const TCHAR* InFilename) override
	{
		AllErrors.Emplace(FText::Format(
			LOCTEXT("Revert", "Failed to revert file '{0}' from revision control when persisting sandbox state!"),
			FText::AsCultureInvariant(InFilename)
			));
	}
	virtual void HandleError_MarkForAdd(const TCHAR* InFilename) override
	{
		AllErrors.Emplace(FText::Format(
			LOCTEXT("MarkForAdd", "Failed to add file '{0}' to revision control when persisting sandbox state!"),
			FText::AsCultureInvariant(InFilename)
			));
	}
	virtual void HandleError_DeleteSCC(const TCHAR* InFilename) override
	{
		AllErrors.Emplace(FText::Format(
			LOCTEXT("DeleteSCC", "Failed to delete file '{0}' from revision control when persisting sandbox state!"),
			FText::AsCultureInvariant(InFilename)
			));
	}
	virtual void HandleError_MakeWritable(const TCHAR* InFilename) override
	{
		AllErrors.Emplace(FText::Format(
			LOCTEXT("MakeWritable", "Failed to make '{0}' writeable using file system when persisting sandbox state!"),
			FText::AsCultureInvariant(InFilename)
			));
	}
	virtual void HandleError_MoveFile(const TCHAR* InToFilename, const TCHAR* InFromFilename) override
	{
		AllErrors.Emplace(FText::Format(
			LOCTEXT("MoveFile", "Failed to move file '{0}' to '{1}' using file system when persisting sandbox state!"),
			FText::AsCultureInvariant(InFromFilename),
			FText::AsCultureInvariant(InToFilename)
			));
	}
	virtual void HandleError_DeleteFile(const TCHAR* InFilename) override
	{
		AllErrors.Emplace(FText::Format(
			LOCTEXT("DeleteFile", "Failed to delete file '{0}' using file system when persisting sandbox state!"),
			FText::AsCultureInvariant(InFilename)
			));
	}
	//~ End IPersistFeedback Interface
};
}

#undef LOCTEXT_NAMESPACE