// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"

class FName;

namespace UE::FileSandboxCore
{
struct FHotReloadPackageArgs;
struct FPurgePackageArgs;

/** Interface to customize hot-reloading and purging of assets. */
class IPackageReloadHandler
{
public:
	
	/** Handles cleaning up assets that no longer exist. */
	virtual void PurgePackages(const FPurgePackageArgs& InArgs) = 0;
	
	/** Reloads packages that were modified */
	virtual void HotReloadPackages(const FHotReloadPackageArgs& InArgs) = 0; 
	
	virtual ~IPackageReloadHandler() = default;
};
}
