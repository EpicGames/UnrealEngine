// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxCore
{
/** You can implement this to get interactive information about a persist operation. */
class IPersistFeedback
{
public:
	
	/** The file was successfully persisted. */
	virtual void StartFile(const TCHAR* InFilename) = 0;
	virtual void HandleSuccess(const TCHAR* InFilename) = 0;

	virtual void HandleError_CheckoutNotAllowed(const TCHAR* InFilename) = 0;
	virtual void HandleError_Checkout(const TCHAR* InFilename) = 0;
	virtual void HandleError_Revert(const TCHAR* InFilename) = 0;
	virtual void HandleError_MarkForAdd(const TCHAR* InFilename) = 0;
	virtual void HandleError_DeleteSCC(const TCHAR* InFilename) = 0;
	
	virtual void HandleError_MakeWritable(const TCHAR* InFilename) = 0;
	virtual void HandleError_MoveFile(const TCHAR* InToFilename, const TCHAR* InFromFilename) = 0;
	virtual void HandleError_DeleteFile(const TCHAR* InFilename) = 0;

	virtual ~IPersistFeedback() = default;
};
}

