// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FText;

namespace UE::FileSandboxCore
{
/** Interface to allow module to determine whether a sandbox can be left. */
class ISandboxLock
{
public:
	
	/**
	 * @param OutReason Optional. Reason set why the sandbox cannot be left.
	 * @return Whether the sandbox can be left
	 */
	virtual bool CanLeaveSandbox(FText* OutReason = nullptr) const = 0;
	
	/** 
	 * Asks the lock to be lifted. The request can be rejected.
	 * @param OutReason Optional. Reason set when the lock could not be lifted.
	 * @return Whether the lock was lifted
	 */
	virtual bool RequestLiftLock(FText* OutReason = nullptr) = 0;
	
	/** @return Whether it is supported to call RequestLiftLock. */
	virtual bool SupportsLiftingLock() const = 0;
	
	virtual ~ISandboxLock() = default;
};
}
