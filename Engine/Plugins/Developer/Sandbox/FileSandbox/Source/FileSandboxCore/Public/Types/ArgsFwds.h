// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FString;

namespace UE::FileSandboxCore
{
struct FDeleteSandboxByDirectoryArgs;
struct FDeleteSandboxByNameArgs;
struct FDeleteSandboxResult;
struct FLeaveSandboxArgs;
struct FLoadSandboxByDirectoryArgs;
struct FLoadSandboxByNameArgs;
struct FNewSandboxArgs;
	
struct FLoadSandboxError;
struct FNewSandboxError;
struct FSandboxCreationResult;
struct FSandboxedFileChangeInfo;
	
template<typename TErrorType>
struct TSandboxCreationResult;
using FNewSandboxResult = TSandboxCreationResult<FNewSandboxError>;
using FLoadSandboxResult = TSandboxCreationResult<FLoadSandboxError>;

enum class EBreakBehavior : uint8;
enum class ESandboxFileChange : uint8;

using FProcessFileChangeSignature = EBreakBehavior(const FSandboxedFileChangeInfo& InFileInfo);
}