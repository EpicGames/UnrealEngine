// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::FileSandboxCore
{
enum class EBreakBehavior : uint8
{
	Continue,
	Break
};
}