// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Misc/DateTime.h"

class FString;

namespace UE::FileSandboxCore
{
enum class ESandboxFileChange : uint8;

/** Specifies what information should be gathered about files */
enum class EFileChangeGatherFlags : uint8
{
	None,
	IncludeChangeTypes = 1 << 0,
	IncludeTimestamps = 1 << 1,
	All = IncludeChangeTypes | IncludeTimestamps
};
ENUM_CLASS_FLAGS(EFileChangeGatherFlags);

/** Result of ISandboxInstance::GatherChangedFiles. */
struct FGatheredFileChanges
{
	/** All files changed by the sandbox. The paths are in the non-sandbox. */
	TArray<FString> NonSandboxPaths;
	
	/** The changes made to the file. Equal length to NonSandboxPaths. Only set if EFileChangeGatherFlags::IncludeChangeTypes was set. */
	TArray<ESandboxFileChange> FileActions;
	
	/** When the change was made, in local time. Only set if EFileChangeGatherFlags::IncludeTimestamps was set.*/
	TArray<FDateTime> Timestamps;
	
	/** @return Whether there are any changes. */
	bool HasChanges() const { return !NonSandboxPaths.IsEmpty(); }
};
}