// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE::FileSandboxCore
{
/** Args for deleting sandbox by directory. */
struct FDeleteSandboxByDirectoryArgs
{
	/** The root directory of the sandbox (i.e. the one containing the manifest file). */
	FString Directory;

	FDeleteSandboxByDirectoryArgs(FString InDirectory) 
		: Directory(MoveTemp(InDirectory))
	{}
};
}