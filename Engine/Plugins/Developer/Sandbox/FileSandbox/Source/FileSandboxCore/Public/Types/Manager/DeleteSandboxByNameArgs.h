// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SandboxInitArgs.h"
#include "Utils/SandboxDirectoryUtils.h"

namespace UE::FileSandboxCore
{
/** Args for deleting sandbox by name. */
struct FDeleteSandboxByNameArgs
{
	/** The name of the sandbox you want to load. */
	FString Name;

	/** The directory in which to search for the sandbox. */
	FString BaseDirectory;

	explicit FDeleteSandboxByNameArgs(FString InName, FString InBaseDirectory = GetBaseSandboxDirectory()) 
		: Name(MoveTemp(InName))
		, BaseDirectory(MoveTemp(InBaseDirectory))
	{
		check(!Name.IsEmpty());
	}
};
}
