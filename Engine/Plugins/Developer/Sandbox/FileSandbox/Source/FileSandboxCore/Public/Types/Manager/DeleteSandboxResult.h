// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::FileSandboxCore
{
enum class EDeleteSandboxErrorCode : uint8
{
	/** Sandbox was deleted successfully. */
	Success,

	/** Given sandbox name could not be resolved to any sandbox. */
	InvalidName,

	/** Given directory does not contain any sandbox. */
	InvalidDirectory,

	/** An I/O error occured, e.g. the directory could not be deleted. */
	IOError,
	
	/**
	 * Failed to leave the sandbox. Most likely because the sandbox is locked.
	 * If you need more info consider calling LeaveSandbox explicitly before.
	 */
	CannotLeaveSandbox,
};

/** Describes the result of the delete operation, with a reason of why it failed. */
struct FDeleteSandboxResult
{
	/** Describes the result of the delete operation, with a reason of why it failed. */
	EDeleteSandboxErrorCode ErrorCode = EDeleteSandboxErrorCode::Success;

	FDeleteSandboxResult() = default;
	FDeleteSandboxResult(EDeleteSandboxErrorCode InErrorCode) : ErrorCode(InErrorCode) {}

	/** @return Whether the sandbox was successfully deleted. */
	bool IsSuccess() const { return ErrorCode == EDeleteSandboxErrorCode::Success; }
};
}