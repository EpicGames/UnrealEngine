// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Types/Sandbox/PersistArgs.h"

namespace UE::FileSandboxCore
{
/** Describes how the session is supposed to be left. */
struct FLeaveSandboxArgs
{
	/** If set, this specifies the files that are supposed to be persisted, and how. */
	TOptional<FPersistArgs> PersistArgs;
};
}
