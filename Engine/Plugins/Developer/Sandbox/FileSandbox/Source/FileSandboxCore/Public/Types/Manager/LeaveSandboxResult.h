// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Optional.h"

namespace UE::FileSandboxCore
{
enum class ELeaveSandboxErrorCode : uint8
{
	/** The sandbox was left successfully, or the engine was not in any sandbox. */
	Success,
	
	/** 
	 * The engine is locked to this sandbox. Inspect the ISandboxLock for the reason.
	 * @see ISandboxManager::GetActiveLock
	 */
	Locked
};

/** Describes the result of the leave operation, with a reason of why it failed. */
struct FLeaveSandboxResult
{
	/** Describes the result of the leave operation, with a reason of why it failed. */
	ELeaveSandboxErrorCode ErrorCode = ELeaveSandboxErrorCode::Success;
	
	FLeaveSandboxResult() = default;
	FLeaveSandboxResult(ELeaveSandboxErrorCode InErrorCode) : ErrorCode(InErrorCode) {}

	/** @return Whether the sandbox was successfully left. */
	bool IsSuccess() const { return ErrorCode == ELeaveSandboxErrorCode::Success; }
	/** @return Whether the operation was successful. */
	operator bool() const { return IsSuccess(); }
};
}