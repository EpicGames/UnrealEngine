// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "SandboxInitArgs.h"

namespace UE::FileSandboxCore
{
/** Args for loading a pre-existing sandbox. */
struct FLoadSandboxByDirectoryArgs
{
	/** The root directory of the sandbox (i.e. the one containing the manifest file). */
	FString SandboxRootPath;
	
	/** Base arguments for instantiating the sandbox environment. */
	FSandboxInitArgs InitArgs;

	explicit FLoadSandboxByDirectoryArgs(FString InSandboxRootPath) : SandboxRootPath(MoveTemp(InSandboxRootPath))
	{
		check(!SandboxRootPath.IsEmpty());
	}
};
}
