// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "LeaveSandboxResult.h"
#include "Misc/Optional.h"

class FString;

namespace UE::FileSandboxCore
{
enum class ELoadSandboxLoadErrorReason : uint8
{
	/** The directory does not contain any sandbox */
	InvalidDirectory,
	/** The name does not map to any directory. */
	InvalidName,
	
	/** Some generic IO error occured (e.g. could not write manifest file, etc.) */
	IOError,
	
	/**
	 * Failed to leave the sandbox. Most likely because the sandbox is locked.
	 * If you need more info consider calling LeaveSandbox explicitly before.
	 */
	CannotLeaveSandbox,

	/**
	 * The sandbox was created with a newer version of Unreal Engine and cannot be loaded.
	 * For example, a 5.8 sandbox cannot be loaded in 5.7, or a sandbox with CL 5819811
	 * cannot be loaded in CL 5719181.
	 */
	IncompatibleVersion,

	/** Dummy error code for initializing to a deterministic value. */
	Unspecified,

	/**
	 * ADD NEW MEMBERS ABOVE.
	 * Not a real reason.
	 */
	Count
};

/** Explains why a new sandbox could not be loaded. */ 
struct FLoadSandboxError
{
	ELoadSandboxLoadErrorReason Reason = ELoadSandboxLoadErrorReason::Unspecified;
};
	
/** @return Error reason converted to a string. */
FILESANDBOXCORE_API FString LexToString(ELoadSandboxLoadErrorReason InReason);
}