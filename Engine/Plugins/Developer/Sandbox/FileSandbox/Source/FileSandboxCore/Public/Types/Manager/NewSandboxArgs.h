// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/SandboxMetaData.h"
#include "SandboxInitArgs.h"
#include "Utils/SandboxDirectoryUtils.h"

namespace UE::FileSandboxCore
{
/** Args for creating a new sandbox. */
struct FNewSandboxArgs
{
	/** Directory in which sandboxes live. A new directory with the sandbox name will be created in this directory. */
	FString SandboxBasePath = GetBaseSandboxDirectory();

	/**
	 * Metadata to set about this new sandbox.
	 * 
	 * The name will be used to create a base directory in the sandbox folder, so it must be non-empty and unique;
	 * invalid name leads to failure.
	 */
	FFileSandboxCore_SandboxMetaData MetaData;
	
	/** Base arguments for instantiating the sandbox environment. */
	FSandboxInitArgs InitArgs;

	explicit FNewSandboxArgs(FString InName, FString InDescription = {})
		: MetaData(MoveTemp(InName), MoveTemp(InDescription))
	{
		check(!MetaData.Name.IsEmpty());
	}
	
	explicit FNewSandboxArgs(FFileSandboxCore_SandboxMetaData InMetaData) : MetaData(MoveTemp(InMetaData))
	{
		check(!MetaData.Name.IsEmpty());
	}
};
}
