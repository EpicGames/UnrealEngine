// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "LeaveSandboxResult.h"
#include "Misc/Optional.h"

class FString;

namespace UE::FileSandboxCore
{
enum class ENewSandboxErrorReason : uint8
{
	/** The specified directory cannot be used as root directory. */
	UnsuitablePath,

	/** Some generic IO error occured (e.g. could not write manifest file, etc.) */
	IOError,

	/**
	 * Failed to leave the sandbox. Most likely because the sandbox is locked.
	 * If you need more info consider calling LeaveSandbox explicitly before.
	 */
	CannotLeaveSandbox,
	
	/** Dummy error code for initializing to a deterministic value. */
	Unspecified,

	/** 
	 * ADD NEW MEMBERS ABOVE. 
	 * Not a real reason. 
	 */
	Count
};

/** Explains why a new sandbox could not be created. */ 
struct FNewSandboxError
{
	ENewSandboxErrorReason Reason = ENewSandboxErrorReason::Unspecified;
};

/** @return Error reason converted to a string. */
FILESANDBOXCORE_API FString LexToString(ENewSandboxErrorReason InReason);
}
