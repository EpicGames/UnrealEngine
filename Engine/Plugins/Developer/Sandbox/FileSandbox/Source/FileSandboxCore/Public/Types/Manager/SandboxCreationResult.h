// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LoadSandboxError.h"
#include "Misc/NotNull.h"
#include "NewSandboxError.h"
#include "Templates/ValueOrError.h"

namespace UE::FileSandboxCore
{
class ISandboxInstance;

/**
 * Utility type to allow implicit conversion to ISandboxInstance*, i.e. to allow shorthand semantics such as
 *		ISandboxManager* Manager = ...;
 *		if (ISandboxInstance* SandboxInstance = Manager->LoadNamedSandbox(...)) { ... }
 */
template<typename TErrorType>
struct TSandboxCreationResult : TValueOrError<TNotNull<ISandboxInstance*>, TErrorType>
{
	using Super = TValueOrError<TNotNull<ISandboxInstance*>, TErrorType>;
	
	TSandboxCreationResult(TValueOrError<TNotNull<ISandboxInstance*>, TErrorType> Result) : Super(MoveTemp(Result)) {}

	operator ISandboxInstance*() const { return this->HasValue() ? this->GetValue() : nullptr; }
};
}
