// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformFileManager.h"
#include "Templates/SharedPointer.h"
#include "Types/SandboxInitFlags.h"

namespace UE::FileSandboxCore
{
class IPackageReloadHandler;
class ISandboxLock;

/** Arguments for creating a sandbox. */
struct FSandboxInitArgs
{
	/**
	 * System for saving out the sandboxed files.
	 * In theory, you could use this to save out the sandboxed files outside of the filesystem, e.g. to the cloud.
	 */
	IPlatformFile* OverridePlatformFile = nullptr;
	
	/** Additional feature flags for the sandbox. */
	ESandboxInitFlags Flags = ESandboxInitFlags::Default;
	
	/**
	 * Optional. You can use this to force the engine to stay in the sandbox until you return true.
	 * Whenever any API call attempts to make the engine leave the sandbox, this is used to check whether the sandbox is allowed to be left.
	 * 
	 * The sandbox system will only keep a reference to this lock for as long as the sandbox is active.
	 */
	TSharedPtr<ISandboxLock> Lock;
	
	/** Optional. You can use this if you want to customize how packages are hot reloaded and purged. */
	TSharedPtr<IPackageReloadHandler> ReloadHandler;

	/** @return The platform file to use. */
	IPlatformFile& GetPlatformFile() const { return OverridePlatformFile ? *OverridePlatformFile : FPlatformFileManager::Get().GetPlatformFile(); }
};
}
