// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "SandboxPackageReloadPhase.h"

class FName;

namespace UE::FileSandboxCore
{
/**
 * Arguments for purging packages.
 * @see IPackageReloadHandler
 */
struct FPurgePackageArgs
{
	/** Provides context for why the packages are being reloaded. */
	ESandboxPackageReloadPhase Phase;
	
	/** The package names being reloaded. */
	TConstArrayView<FName> PackageNames;

	explicit FPurgePackageArgs(ESandboxPackageReloadPhase Phase, const TConstArrayView<FName>& PackageNames)
		: Phase(Phase)
		, PackageNames(PackageNames)
	{}
};
}
