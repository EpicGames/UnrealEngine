// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::FileSandboxCore
{
/** Specifies a stage at which packages may be reloaded while sandboxed. */
enum class ESandboxPackageReloadPhase : uint8
{
	/** Sandbox is starting up and bringing files into the state of the sandbox.*/
	Startup,
	/** The engine is sandboxed and reloading files as part of an operation, e.g. reverting a change. */
	Sandboxed,
	/** Sandbox is shutting down and cleaning up packages. */
	Shutdown
};
}
