// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"

namespace UE::FileSandboxCore
{
struct FPersistedFileInfo
{
	/** File in the original file system (e.g. "D:/MyProject/Content/Levels/Map.umap"). */
	FString FileName;

	/** Timestamp of when the file was persisted (to detect if the file has been changed again after it was persisted). */
	FDateTime Timestamp;
};
}
