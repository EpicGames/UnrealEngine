// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Containers/ArrayView.h"

namespace UE::FileSandboxCore
{
/** Data describing a change in a repository */
struct FRepositoryChangedEvent
{
	/** Root directories of sandboxes that have been added. */
	TConstArrayView<FString> AddedSandboxPaths;
	
	/** Root directories of sandboxes that have been removed. */
	TConstArrayView<FString> RemovedSandboxPaths;
};
}
