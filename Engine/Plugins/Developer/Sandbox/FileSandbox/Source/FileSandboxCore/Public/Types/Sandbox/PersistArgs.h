// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"

class FString;

namespace UE::FileSandboxCore
{
class IPersistFeedback;

/** Input arguments for persisting a sandbox. */
struct FPersistArgs
{
	/** List of files names from the original filesystem, e.g. "D:/MyProject/Content/Levels/MyLevel.umap" */
	TConstArrayView<FString> Files;
	
	/** Whether to force the file writable when no source control provider is active. */
	bool bShouldMakeWritableIfNoSourceControl = false;
	
	/** 
	 * You can implement this to get interactive information about a persist operation.
	 * Useful for displaying a message about the persist operation or a progress bar. 
	 * 
	 * You can aggregate actions.
	 * @see FAggregatePersistFeedback
	 */
	IPersistFeedback* Feedback = nullptr;
};
}
