// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Internationalization/Text.h"
#include "HAL/Platform.h"

class ISourceControlProvider;

namespace UE::FileSandboxCore
{
/** Possible outcomes from a session persist. */
enum class EPersistStatus : uint8
{
	/** The packages were persisted correctly. */
	Success,
	/** We failed to persist the packages. */
	Failure,
	/** We are not allowed to persist. */
	NotAllowed
};

/** Result of a persist.*/
struct FPersistResult
{
	/** Success or failure of the session persist. */
	EPersistStatus PersistStatus = EPersistStatus::NotAllowed;
};
}
