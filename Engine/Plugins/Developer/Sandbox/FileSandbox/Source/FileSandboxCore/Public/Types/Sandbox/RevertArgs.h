// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

class FString;

namespace UE::FileSandboxCore
{
/** Input arguments for reverting a sandbox. */
struct UE_EXPERIMENTAL(5.8, "Reverting of single files does not yet work for all cases. See UE-368478.") FRevertArgs
{
	/** 
	 * Non-sandbox paths to files that should be reverted.
	 * If this is left empty, then the operation reverts all file changes.
	 * 
	 * This can be paths to files with any type of change (i.e. removed, edited, added).
	 * The path can be absolute or relative.
	 */
	TConstArrayView<FString> FilesToRevert;
	
	FRevertArgs() = default;
	explicit FRevertArgs(TConstArrayView<FString> InFilesToRevert) 
		: FilesToRevert(InFilesToRevert) 
	{}
};
}
