// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/NameTypes.h"

namespace UE::FileSandboxCore
{
/** Result of reverting a sandbox. */
struct FRevertResult
{
	/**
	 * The package names that should be hot reloaded.
	 * You must call HotReloadPackages on them.
	 */
	TArray<FName> PackagesPendingHotReload;
	
	/**
	 * The package names that should be purged.
	 * You must call PurgePackages on them.
	 */
	TArray<FName> PackagesPendingPurge;
};
}
