// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::FileSandboxCore
{
/** Types of changes the sandbox system may make to a file. */
enum class ESandboxFileChange : uint8
{
	/** No changes were made to the file. */
	None,
	
	/** The file was added. */
	Added,
	/** The file was removed. */
	Removed,
	/** The file was edited. */
	Edited
	
	// TODO UE-356393: Add support for moved files & moved-and-edited files.
};
}