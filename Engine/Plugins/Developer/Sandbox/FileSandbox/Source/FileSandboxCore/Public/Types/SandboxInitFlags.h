// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::FileSandboxCore
{
enum class ESandboxInitFlags : uint8
{
	None,
	
	/** 
	 * If the user is using source control, then renaming assets should cause redirectors to be created. 
	 * Under the hood, it forces ISourceControlState::IsLocal() to return false.
	 */
	ForceRedirectorsOnRenameInSourceControl = 1 << 0,
	
	Default = ForceRedirectorsOnRenameInSourceControl
};

ENUM_CLASS_FLAGS(ESandboxInitFlags);
}