// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/SandboxMetaData.h"
#include "Data/VersionInfo.h"
#include "Misc/DateTime.h"

namespace UE::FileSandboxCore
{
struct FSandboxMetaInfo
{
	/** User / API-user provided meta data. */
	FFileSandboxCore_SandboxMetaData UserMetaData;
	
	/** Local time that the sandbox was last modified. */
	FDateTime LastModified;
	/** Version that the sandbox was created in. */
	FFileSandboxCore_VersionInfo VersionInfo;

	explicit FSandboxMetaInfo(
		FFileSandboxCore_SandboxMetaData InMetaData, FDateTime InLastModified, FFileSandboxCore_VersionInfo InVersionInfo
		)
		: UserMetaData(MoveTemp(InMetaData)), LastModified(InLastModified), VersionInfo(MoveTemp(InVersionInfo))
	{}
};
}
