// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/DateTime.h"
#include "SandboxFileChange.h"

namespace UE::FileSandboxCore
{
/**
 * Flags enumerating file changes.
 * @see ISandboxManager::EnumerateFileChanges
 * @see ISandboxInstance::EnumerateFileChanges
 */
enum class EFileEnumerationFlags : uint8
{
	None,
	IncludeTimestamps = 1 << 0,
	All = IncludeTimestamps
};
ENUM_CLASS_FLAGS(EFileEnumerationFlags);

/**
 * Info about each file change enumerated.
 * @see ISandboxManager::EnumerateFileChanges
 * @see ISandboxInstance::EnumerateFileChanges
 */
struct FSandboxedFileChangeInfo
{
	/** Absolute non-sandbox file path. */
	FString Path;
	
	/** The action performed on the file */
	ESandboxFileChange Action = ESandboxFileChange::None;
	
	/** Only set if EFileEnumerationFlags::IncludeTimestamps was specified */
	FDateTime Timestamp = FDateTime::MinValue();
};
}
