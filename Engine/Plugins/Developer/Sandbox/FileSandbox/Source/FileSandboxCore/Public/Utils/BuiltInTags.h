// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::FileSandboxCore
{
/** Tag added to sandboxes that were created from command line. */
constexpr FLazyName Tag_CommandLineSandbox("CommandLineSandbox");
}
