// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#define UE_API FILESANDBOXCORE_API

class FName;

namespace UE::FileSandboxCore
{
/** Handles cleaning up assets that no longer exist. */
UE_API void PurgePackages(const TConstArrayView<FName>& InPackageNames);

/** Reloads packages that were modified */
UE_API bool HotReloadPackages(const TConstArrayView<FName>& InPackageNames);
UE_API bool HotReloadPackages(const TConstArrayView<UPackage*>& InPackages);

/** @return Packages with in-memory changes. */
UE_API TArray<UPackage*> GetDirtyPackages();
/** @return Whether there are packages with in-memory changes. */
inline bool HasDirtyPackages() { return !GetDirtyPackages().IsEmpty(); }

/**
 * If the current editor world's persistent level uses external objects (i.e. a World Partition level),
 * schedule its package for hot reload. This ensures the full WP state is rebuilt from disk when a sandbox
 * is discarded, clearing any in-memory actors whose per-actor packages only existed in the sandbox.
 *
 * No-op if the package is already scheduled for purge or already in the hot-reload list, or if the current
 * world's persistent level is not using external objects.
 */
UE_API void AppendExternalPersistentLevelForReload(
	TArray<FName>& InOutPackagesPendingHotReload,
	TConstArrayView<FName> InPackagesPendingPurge);
}

#undef UE_API