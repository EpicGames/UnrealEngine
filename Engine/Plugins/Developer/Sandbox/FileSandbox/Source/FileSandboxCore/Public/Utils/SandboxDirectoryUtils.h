// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Templates/FunctionFwd.h"

#define UE_API FILESANDBOXCORE_API

class FText;
struct FFileSandboxCore_VersionInfo;

struct FFileSandboxCore_SandboxMetaData;

namespace UE::FileSandboxCore
{
enum class EBreakBehavior : uint8;
using FProcessSandboxDirectorySignature = EBreakBehavior(const FString& InBaseDirectory);

/** @return The base directory into which new sandboxes are stored. */
UE_API FString GetBaseSandboxDirectory();


/**
 * Enumerates all directories that contain a sandbox. The search only considers the top-level directories.
 * @param InProcessSandbox Invokes for each found sandbox.
 * @param InBaseDirectory Base directory search for. Either absolute or relative to the process BaseDir().
 */
UE_API void ForEachSandbox(
	TFunctionRef<FProcessSandboxDirectorySignature> InProcessSandbox,
	const FString& InBaseDirectory = GetBaseSandboxDirectory()
	);

/** @return Whether this is a path to the root of a sandbox, i.e. whether it contains the manifest and the metadata file. */
UE_API bool IsRootSandboxDirectory(const FString& InRootDirectory);


/**
 * Writes new metadata for a sandbox.
 * 
 * You are allowed to change only the sandbox name. However, it will not rename the sandbox directory.
 * To rename the directory, @see RenameSandboxWithDirectory.
 * 
 * @param NewMetaData The new metadata to set for the sandbox.
 * @param InSandboxDirectory Path to the base directory of the sandbox, e.g. what's returned by ForEachSandbox. Either absolute or relative to the process BaseDir().
 */
UE_API bool SaveMetaData(
	const FFileSandboxCore_SandboxMetaData& NewMetaData, const FString& InSandboxDirectory
	);
	
/**
 * Reads metadata about a sandbox.
 * @param InSandboxDirectory Path to the base directory of the sandbox, e.g. what's returned by ForEachSandbox. Either absolute or relative to the process BaseDir().
 */
UE_API TOptional<FFileSandboxCore_SandboxMetaData> LoadMetaData(const FString& InSandboxDirectory);

/**
 * Renames the sandbox located in InBaseDirectory / InOldName. 
 * 
 * By default, sandboxes are placed in BaseDir / SandboxName. This function renames the containing directory to the new name.
 * This also updates the manifest file with the new name.
 * 
 * @param InNewName The new name the sandbox should have
 * @param InOldName The sandbox you want to rename. The sandbox root is located in InBaseDirectory / InOldName directory.
 * @param InBaseDirectory Base directory search for. Either absolute or relative to the process BaseDir().
 * @param InNewMetaData Optional additional metadata you want to set. Specifying this will avoid loading the metadata from disk first.
 * @return Whether the sandbox was successfully renamed
 */
UE_API bool RenameSandboxWithDirectory(
	const FString& InNewName, const FString& InOldName, 
	const FString& InBaseDirectory = GetBaseSandboxDirectory(),
	const FFileSandboxCore_SandboxMetaData* InNewMetaData = nullptr
	);

/** Explains why a sandbox can or cannot be renamed. */
enum class ESandboxRenameSuitability : uint8
{
	Allowed,
	
	/** The sandbox does not exist */
	SandboxDoesNotExist,
	/** You cannot rename an active sandbox */
	ActiveSandbox,
	
	/** The name is not different from the current name. */
	NameIsSame,
	/** Sandbox cannot have this name */
	InvalidDirectory,
	/** Name cannot be empty */
	EmptyName,
};

/** @return Whether InNewName is a valid new name for the sandbox located in InBaseDirectory / InOldName. */
UE_API ESandboxRenameSuitability CanRenameSandboxWithDirectory(
	const FString& InNewName, const FString& InOldName, const FString& InBaseDirectory = GetBaseSandboxDirectory()
	);
/** @return Whether the given sandbox can be renamed, i.e. it is not active. */
UE_API bool IsAllowedToRenameSandbox(const FString& InSandboxDirectory);


/** @return When the given sandbox was last modified. Returns FDateTime::MinValue() on failure. */
UE_API FDateTime GetLastModified(const FString& InSandboxDirectory);

/** @return Version info for the sandbox saved in this directory. This loads the data from the manifest file. IsInitialized returns false if this fails. */
UE_API FFileSandboxCore_VersionInfo LoadVersionInfo(const FString& InSandboxDirectory);

	
/**
 * Searches for sandboxes with the given name. The search only considers the top-level directories.
 * 
 * @param InSandboxName The name of the sandbox
 * @param InBaseDirectory Base directory search for. Either absolute or relative to the process BaseDir().
 * @return The path to the directory containing the sandbox with the specified name.
 */
UE_API TOptional<FString> FindSandboxByName(
	const FString& InSandboxName,
	const FString& InBaseDirectory = GetBaseSandboxDirectory()
	);


	
enum class EDirectorySuitability : uint8
{
	Suitable,
	EmptyName,
	NameTooLong,
	DuplicateName,
	InvalidPath,
	DirectoryAlreadyExists
};
	
/** @return Whether a sandbox directory named InSandboxName can be put into the InBaseDirectory. */
UE_API EDirectorySuitability DetermineDirectorySuitability(
	const FString& InSandboxName,
	const FString& InBaseDirectory = GetBaseSandboxDirectory(),
	FText* OutReason = nullptr
	);

/** @return EDirectorySuitability explained as reason to the user. */
UE_API FText FormatDirectorySuitabilityAsReason(
	EDirectorySuitability Suitability,
	const FString& InSandboxName,
	const FString& InBaseDirectory = GetBaseSandboxDirectory()
	);

/** 
 * Gets the file path to the file that saves the sandboxed state of InNonSandboxPath. 
 * This only works for edited and added files. For removed files, this returns nothing.
 * 
 * @param InSandboxDirectory Path to the base directory of the sandbox, e.g. what's returned by ForEachSandbox. Either absolute or relative to the process BaseDir().
 * @param InNonSandboxPath Path to a non-sandbox file of which you want to get the sandbox file.
 * @return Path to sandboxed file corresponding to the non-sandbox file, if the file has a sandboxed edit or add operation. Unset, otherwise.
 */
UE_API TOptional<FString> GetSandboxPathFor(const FString& InSandboxDirectory, const FString& InNonSandboxPath);

/** @return Name of the sandbox if the directory is the root of a sandbox. Unset otherwise. */
UE_API TOptional<FString> GetSandboxName(const FString& InSandboxDirectory);
}

#undef UE_API