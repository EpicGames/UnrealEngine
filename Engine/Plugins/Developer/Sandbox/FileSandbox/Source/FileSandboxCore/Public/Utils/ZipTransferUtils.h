// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Data/SandboxMetaData.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"

class FString;
class FZipArchiveReader;

#define UE_API FILESANDBOXCORE_API

namespace UE::FileSandboxCore::ZipUtils
{
/** Zips up a sandbox and saves it to the destination. */
UE_API bool ExportSandboxToZip(const FString& InSandboxRoot, const FString& InOutputZipPath);

/**
 * Holds the result for inspecting whether a zip archive is suitable for importing.
 * This also contains data produced as side effects, which can be reused for importing. 
 */
struct FImportInspectionResult
{
	/** The zip reader that was produced as side effect. You can re-use this for the import operation. */
	TUniquePtr<FZipArchiveReader> Reader; // FZipArchiveReader does not support move semantics so we must wrap it with TUniquePtr.
	/** All file names in the archive. Produced as side effect. */
	TArray<FString> AllFileNames;
	
	/** The name of the sandbox contained in the zip. Produced as side effect. */
	FString SandboxName;
	/** Manifest file content. Produced as side effect. */
	TArray<uint8> ManifestBytes;
	/** Metadata file content. Produced as side effect. */
	TArray<uint8> MetadataBytes;
	/** Metadata. Produced as side effect. */
	TOptional<FFileSandboxCore_SandboxMetaData> MetaData;
	
	explicit FImportInspectionResult(TUniquePtr<FZipArchiveReader> InReader);
	
	/** @return Whether the zip archive is valid to import from. */
	bool IsValid() const { return !SandboxName.IsEmpty() && MetaData.IsSet(); }
};

/** 
 * Inspects whether a zip contains a sandbox valid for importing, i.e.
 * - the zip must be valid,
 * - must contain exactly one root directory with the name of the sandbox,
 * - must contain required files (manifest and metadata) 
 * @return Result of inspecting the zip. Unset if I/O errors occurs.
 * @note Creates IFileHandle that FZipArchiveReader takes ownership of until destroyed.
 */
UE_API TOptional<FImportInspectionResult> InspectFileForImport(const FString& InPathToZip);

/** 
 * Unzips a zip and imports it to the sandbox system reusing any data produced during inspection. 
 * @note Pre-existing files in the target directory will be deleted.
 */
UE_API bool ImportSandboxFromZip(const FImportInspectionResult& InInspection, const FString& InSandboxTargetRoot);
}

#undef UE_API
