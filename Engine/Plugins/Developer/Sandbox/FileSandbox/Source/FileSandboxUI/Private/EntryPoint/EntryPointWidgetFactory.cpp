// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntryPoint/EntryPointWidgetFactory.h"

#include "ExternalSandboxActiveViewModel.h"
#include "Widgets/SExternalSandboxActiveOverlay.h"

namespace UE::FileSandboxUI
{
TSharedRef<SWidget> MakeExternalSandboxActiveOverlay(const TSharedRef<IExternalSandboxActiveViewModel>& InViewModel)
{
	return SNew(SExternalSandboxActiveOverlay, InViewModel);
}

TSharedRef<IExternalSandboxActiveViewModel> MakeExternalSandboxActiveViewModel(
	const TSharedRef<ISandboxEntryPoint>& InOwningEntryPoint
	)
{
	return MakeShared<FExternalSandboxActiveViewModel>(InOwningEntryPoint);
}
}
