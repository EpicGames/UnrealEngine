// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalSandboxActiveViewModel.h"

#include "EntryPoint/ISandboxEntryPoint.h"
#include "EntryPoint/ISandboxEntryPointRegistry.h"
#include "IFileSandboxCoreModule.h"
#include "IFileSandboxUIModule.h"
#include "ISandboxManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "FExternalSandboxActiveViewModel"

namespace UE::FileSandboxUI
{
FExternalSandboxActiveViewModel::FExternalSandboxActiveViewModel(
	const TSharedRef<ISandboxEntryPoint>& InOwningEntryPoint
	)
	: OwningEntryPoint(InOwningEntryPoint)
{
	using namespace UE::FileSandboxCore;
	IFileSandboxCoreModule& ModuleCore = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = ModuleCore.GetSandboxManager();
	SandboxManager.OnPostSandboxStartup().AddRaw(this, &FExternalSandboxActiveViewModel::OnSandboxStartup);
	SandboxManager.OnPostSandboxShutdown().AddRaw(this, &FExternalSandboxActiveViewModel::OnSandboxShutdown);
}

FExternalSandboxActiveViewModel::~FExternalSandboxActiveViewModel()
{
	using namespace UE::FileSandboxCore;
	if (IFileSandboxCoreModule::IsAvailable())
	{
		IFileSandboxCoreModule& ModuleCore = IFileSandboxCoreModule::Get();
		ISandboxManager& SandboxManager = ModuleCore.GetSandboxManager();
		SandboxManager.OnPostSandboxStartup().RemoveAll(this);
		SandboxManager.OnPostSandboxShutdown().RemoveAll(this);
	}
}

bool FExternalSandboxActiveViewModel::IsExternalSandboxActive() const
{
	using namespace UE::FileSandboxCore;
	IFileSandboxCoreModule& ModuleCore = IFileSandboxCoreModule::Get();
	IFileSandboxUIModule& ModuleUI = IFileSandboxUIModule::Get();
	return ModuleCore.GetSandboxManager().HasActiveSandbox() 
		&& ModuleUI.GetEntryPointRegistry().FindOwnerOfActiveSandbox() != OwningEntryPoint;
}

void FExternalSandboxActiveViewModel::SummonSandboxOwnerUI() const
{
	IFileSandboxUIModule& ModuleUI = IFileSandboxUIModule::Get();
	if (const TSharedPtr<ISandboxEntryPoint> EntryPoint = ModuleUI.GetEntryPointRegistry().FindOwnerOfActiveSandbox())
	{
		EntryPoint->SummonProviderUI();
	}
}

bool FExternalSandboxActiveViewModel::IsSummoningSupported() const
{
	IFileSandboxUIModule& ModuleUI = IFileSandboxUIModule::Get();
	const TSharedPtr<ISandboxEntryPoint> EntryPoint = ModuleUI.GetEntryPointRegistry().FindOwnerOfActiveSandbox();
	return EntryPoint && EntryPoint != OwningEntryPoint;
}

FText FExternalSandboxActiveViewModel::GetSummonActionLabel() const
{
	IFileSandboxUIModule& ModuleUI = IFileSandboxUIModule::Get();
	const TSharedPtr<ISandboxEntryPoint> EntryPoint = ModuleUI.GetEntryPointRegistry().FindOwnerOfActiveSandbox();
	return EntryPoint ? EntryPoint->GetEntryPointLabel() : FText::GetEmpty();
}

FText FExternalSandboxActiveViewModel::GetExternalSandboxActiveText() const
{
	IFileSandboxUIModule& ModuleUI = IFileSandboxUIModule::Get();
	
	if (const TSharedPtr<ISandboxEntryPoint> EntryPoint = ModuleUI.GetEntryPointRegistry().FindOwnerOfActiveSandbox())
	{
		return FText::Format(
			LOCTEXT("Unavailable.KnownEntryPointFmt", "{0} is unavailable because the engine is sandboxed by {1}."),
			OwningEntryPoint->GetEntryPointLabel(),
			EntryPoint->GetEntryPointLabel()
			);
	}
	
	return FText::Format(
		LOCTEXT("Unavailable.UnknownEntryPointFmt", "{0} is unavailable because the engine is sandboxed."),
		OwningEntryPoint->GetEntryPointLabel()
		);
}
}

#undef LOCTEXT_NAMESPACE