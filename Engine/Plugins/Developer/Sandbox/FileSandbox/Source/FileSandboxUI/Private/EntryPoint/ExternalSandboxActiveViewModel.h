// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISandboxInstance.h"
#include "EntryPoint/IExternalSandboxActiveViewModel.h"

namespace UE::FileSandboxUI
{
class ISandboxEntryPoint;

/** This view-model is associated with an ISandboxEntryPoint. The UI is shown when the sandbox is not owned by its entry point. */
class FExternalSandboxActiveViewModel : public IExternalSandboxActiveViewModel
{
public:
	
	explicit FExternalSandboxActiveViewModel(const TSharedRef<ISandboxEntryPoint>& InOwningEntryPoint);
	virtual ~FExternalSandboxActiveViewModel() override;
	
	//~ Begin IExternalSandboxActiveViewModel Interface
	virtual bool IsExternalSandboxActive() const override;
	virtual void SummonSandboxOwnerUI() const override;
	virtual bool IsSummoningSupported() const override;
	virtual FText GetSummonActionLabel() const override;
	virtual FText GetExternalSandboxActiveText() const override;
	virtual FSimpleMulticastDelegate& OnVisibilityChanged() override { return OnVisibilityChangedDelegate; }
	//~ End IExternalSandboxActiveViewModel Interface
	
private:
	
	/** The entry point for which the UI is being displayed. Used to determine whether an external sandbox is active. */
	const TSharedRef<ISandboxEntryPoint> OwningEntryPoint;
	
	/** Event invoked when the visibility of the widget should change. */
	FSimpleMulticastDelegate OnVisibilityChangedDelegate;
	
	void OnSandboxStartup(FileSandboxCore::ISandboxInstance& SandboxInstance) { return OnVisibilityChangedDelegate.Broadcast(); }
	void OnSandboxShutdown() { return OnVisibilityChangedDelegate.Broadcast(); }
};
}

