// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxEntryPointManager.h"

#include "EntryPoint/ISandboxEntryPoint.h"
#include "IFileSandboxCoreModule.h"
#include "ISandboxManager.h"

namespace UE::FileSandboxUI
{
void FSandboxEntryPointManager::RegisterEntryPoint(const TSharedRef<ISandboxEntryPoint>& InEntryPoint)
{
	EntryPoints.AddUnique(InEntryPoint);
	OnEntryPointsChangedDelegate.Broadcast();
}

void FSandboxEntryPointManager::UnregisterEntryPoint(const TSharedRef<ISandboxEntryPoint>& InEntryPoint)
{
	EntryPoints.RemoveSingle(InEntryPoint);
	OnEntryPointsChangedDelegate.Broadcast();
}

TSharedPtr<ISandboxEntryPoint> FSandboxEntryPointManager::FindOwnerOfActiveSandbox() const
{
	using namespace UE::FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	const ISandboxInstance* Instance = Module.GetSandboxManager().GetActiveSandboxInstance();
	if (!Instance)
	{
		return nullptr;
	}
	
	// We don't handle multiple EntryPoints reporting ownership.
	const int32 Index = EntryPoints.IndexOfByPredicate([Instance](const TWeakPtr<ISandboxEntryPoint>& WeakEntryPoint)
	{
		const TSharedPtr<ISandboxEntryPoint> EntryPointPin = WeakEntryPoint.Pin();
		return EntryPointPin && EntryPointPin->OwnsSandbox(*Instance);
	});
	return EntryPoints.IsValidIndex(Index) ? EntryPoints[Index].Pin() : nullptr;
}
}
