// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EntryPoint/ISandboxEntryPointRegistry.h"

namespace UE::FileSandboxUI
{
class FSandboxEntryPointManager : public ISandboxEntryPointRegistry
{
public:
	
	//~ Begin ISandboxEntryPointRegistry Interface
	virtual void RegisterEntryPoint(const TSharedRef<ISandboxEntryPoint>& InEntryPoint) override;
	virtual void UnregisterEntryPoint(const TSharedRef<ISandboxEntryPoint>& InEntryPoint) override;
	virtual TSharedPtr<ISandboxEntryPoint> FindOwnerOfActiveSandbox() const override;
	virtual FSimpleMulticastDelegate& OnEntryPointsChanged() override { return OnEntryPointsChangedDelegate; }
	//~ End ISandboxEntryPointRegistry Interface
	
private:
	
	/** The entry points this manager knows about. */
	TArray<TWeakPtr<ISandboxEntryPoint>> EntryPoints;
	
	/** Event invoked when the registered entry points have changed. */
	FSimpleMulticastDelegate OnEntryPointsChangedDelegate;
};
}

