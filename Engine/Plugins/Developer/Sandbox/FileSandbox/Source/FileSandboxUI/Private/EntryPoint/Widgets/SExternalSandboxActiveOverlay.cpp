// Copyright Epic Games, Inc. All Rights Reserved.

#include "SExternalSandboxActiveOverlay.h"

#include "Components/VerticalBox.h"
#include "EntryPoint/IExternalSandboxActiveViewModel.h"
#include "FileSandboxStyle.h"
#include "SActionButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SExternalSandboxActiveOverlay"

namespace UE::FileSandboxUI
{
void SExternalSandboxActiveOverlay::Construct(const FArguments& InArgs, const TSharedRef<IExternalSandboxActiveViewModel>& InViewModel)
{
	ViewModel = InViewModel;
	
	ChildSlot
	[
		SNew(SOverlay)

		+SOverlay::Slot()
		.ZOrder(0)
		[
			SNew(SImage)
			.Image(FFileSandboxStyle::Get().GetBrush("FileSandbox.TranslucentBackground"))
		]

		+SOverlay::Slot()
		.ZOrder(1)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(this, &SExternalSandboxActiveOverlay::GetTitleText)
				.Justification(ETextJustify::Center)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0.f, 10.f)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SActionButton)
					.Text(this, &SExternalSandboxActiveOverlay::GetSummonButtonLabel)
					.Visibility_Lambda([this]{ return ViewModel->IsSummoningSupported() ? EVisibility::Visible : EVisibility::Collapsed; })
					.OnClicked(this, &SExternalSandboxActiveOverlay::OnSummonClicked)
				]
			]
		]
	];
	
	// By setting visiblity here, we avoid duplicate visiblity logic for all API users.
	SetVisibility(TAttribute<EVisibility>::CreateLambda([this]
	{
		return ViewModel->IsExternalSandboxActive() ? EVisibility::Visible : EVisibility::Collapsed;
	}));
}

FText SExternalSandboxActiveOverlay::GetTitleText() const
{
	return ViewModel->GetExternalSandboxActiveText();
}

FText SExternalSandboxActiveOverlay::GetSummonButtonLabel() const
{
	return FText::Format(LOCTEXT("SummonUI.Label", "Go to {0}"), ViewModel->GetSummonActionLabel());
}

FReply SExternalSandboxActiveOverlay::OnSummonClicked()
{
	ViewModel->SummonSandboxOwnerUI();
	return FReply::Handled();
}
}

#undef LOCTEXT_NAMESPACE