// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::FileSandboxUI
{
class IExternalSandboxActiveViewModel;

/**
 * Use this overlay in your editor system to tell the user that the engine is sandboxed by another editor system.
 * This widget shows a button to summon the other system's UI.
 */
class SExternalSandboxActiveOverlay : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SExternalSandboxActiveOverlay) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IExternalSandboxActiveViewModel>& InViewModel);
	
private:
	
	/** View model that we're displaying. */
	TSharedPtr<IExternalSandboxActiveViewModel> ViewModel;
	
	FText GetTitleText() const;
	FText GetSummonButtonLabel() const;
	FReply OnSummonClicked();
};
}

