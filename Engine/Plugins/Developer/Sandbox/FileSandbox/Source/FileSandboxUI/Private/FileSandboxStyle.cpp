// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileSandboxStyle.h"

#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::FileSandboxUI
{
FFileSandboxStyle& FFileSandboxStyle::Get()
{
	static FFileSandboxStyle Style;
	return Style;
}

FFileSandboxStyle::FFileSandboxStyle()
	: FSlateStyleSet("FileSandboxStyle")
{
	const FString PluginContentDir = FPaths::EnginePluginsDir() / TEXT("Developer") / TEXT("Sandbox") / TEXT("FileSandboxCore") / TEXT("Content");
	const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor") / TEXT("Slate");
	SetContentRoot(PluginContentDir);
	SetCoreContentRoot(EngineEditorSlateDir);
	
	Set("FileSandbox.TranslucentBackground", new FSlateColorBrush(FLinearColor(0.f, 0.f, 0.f, 0.6f)));
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FFileSandboxStyle::~FFileSandboxStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
}
