// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FSlateStyleSet;
class ISlateStyle;

namespace UE::FileSandboxUI
{
class FFileSandboxStyle final : public FSlateStyleSet
{
public:
	
	static FFileSandboxStyle& Get();

	FFileSandboxStyle();
	virtual ~FFileSandboxStyle() override;
};
}

