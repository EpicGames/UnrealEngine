// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileSandboxUIModule.h"

#include "EntryPoint/SandboxEntryPointManager.h"

namespace UE::FileSandboxUI
{
void FFileSandboxUIModule::StartupModule()
{
    EntryPoints = MakeUnique<FSandboxEntryPointManager>();
}

void FFileSandboxUIModule::ShutdownModule()
{
    EntryPoints.Reset();
}
}
    
IMPLEMENT_MODULE(UE::FileSandboxUI::FFileSandboxUIModule, FileSandboxUI)