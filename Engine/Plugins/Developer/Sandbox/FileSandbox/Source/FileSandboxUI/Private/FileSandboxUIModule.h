// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IFileSandboxUIModule.h"
#include "EntryPoint/SandboxEntryPointManager.h"

namespace UE::FileSandboxUI
{
/** Knows about editor systems that start sandboxes. */
class FFileSandboxUIModule : public IFileSandboxUIModule
{
public:
	
	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface
	
	//~ Begin IFileSandboxUIModule Interface
	virtual ISandboxEntryPointRegistry& GetEntryPointRegistry() override { return *EntryPoints.Get(); }
	//~ End IFileSandboxUIModule Interface
	
private:
	
	/** Knows about editor systems that start sandboxes. */
	TUniquePtr<FSandboxEntryPointManager> EntryPoints;
};
}

