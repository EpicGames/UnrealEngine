// Copyright Epic Games, Inc. All Rights Reserved.

#include "Persist/Feedback/SummaryPersistFeedback.h"

#include "Misc/Paths.h"
#include "Types/SandboxFileChange.h"

namespace UE::FileSandboxUI
{
FSummaryPersistFeedback::FSummaryPersistFeedback(
	TConstArrayView<FString> InPersistedFiles,
	TConstArrayView<FileSandboxCore::ESandboxFileChange> InFileActions
	)
{
	check(InPersistedFiles.Num() == InFileActions.Num());
	for (int32 Index = 0; Index < InPersistedFiles.Num(); ++Index)
	{
		const FileSandboxCore::ESandboxFileChange Change = InFileActions[Index];
		FileActions.Add(FPaths::ConvertRelativePathToFull(InPersistedFiles[Index]), Change);
	}
}

void FSummaryPersistFeedback::HandleSuccess(const TCHAR* InFilename)
{
	FString Abs = FPaths::ConvertRelativePathToFull(InFilename);
	
	const FileSandboxCore::ESandboxFileChange* FileChange = FileActions.Find(Abs);
	if (ensure(FileChange))
	{
		switch (*FileChange)
		{
		case FileSandboxCore::ESandboxFileChange::Added: 
			++Summary.Added;
			break;
		case FileSandboxCore::ESandboxFileChange::Removed:
			++Summary.Deleted;
			break;
		case FileSandboxCore::ESandboxFileChange::Edited:
			++Summary.Edited;
			break;
		case FileSandboxCore::ESandboxFileChange::None: [[fallthrough]];
		default: ensure(false);
		}
	}
}

void FSummaryPersistFeedback::HandleError(const TCHAR* InToFilename)
{
	FString Abs = FPaths::ConvertRelativePathToFull(InToFilename);
	Summary.FailedFiles.Emplace(MoveTemp(Abs));
}
}
