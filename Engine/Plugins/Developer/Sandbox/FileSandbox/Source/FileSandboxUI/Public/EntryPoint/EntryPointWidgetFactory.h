// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

#define UE_API FILESANDBOXUI_API

class FText;
class SWidget;

namespace UE::FileSandboxUI
{
class ISandboxEntryPoint;
class IExternalSandboxActiveViewModel;

/** @return Widget to overlay your UI with when you want to tell the user that a sandbox owned by another editor system is blocking your feature. */
UE_API TSharedRef<SWidget> MakeExternalSandboxActiveOverlay(const TSharedRef<IExternalSandboxActiveViewModel>& InViewModel);

/** @return A view model for the entry point. */
UE_API TSharedRef<IExternalSandboxActiveViewModel> MakeExternalSandboxActiveViewModel(
	const TSharedRef<ISandboxEntryPoint>& InOwningEntryPoint
	);
}

#undef UE_API