// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"

namespace UE::FileSandboxUI
{
/** 
 * This view-model is used for UI telling the user that a sandbox cannot currently be started.
 * Usually, the widget displays a message that another editor system has started a sandbox and provides a way to summon the UI of the other system. 
 */
class IExternalSandboxActiveViewModel
{
public:
	
	/** @return Whether the mutex widget should be displayed, i.e. a sandbox is active and not owned by the current ISandboxEntryPoint. */
	virtual bool IsExternalSandboxActive() const = 0;
	
	/** Summons the UI of the system owning the active sandbox. */
	virtual void SummonSandboxOwnerUI() const = 0;
	/** @return Whether the UI (e.g. button) for summoning the sandbox owner should be shown. */
	virtual bool IsSummoningSupported() const = 0;
	/** @return The label to display for the summon action, e.g. could be displayed in button. */
	virtual FText GetSummonActionLabel() const = 0;
	
	/** @return A text explaining the current system is not available while the external system is active. Empty of no external sandbox is active. */
	virtual FText GetExternalSandboxActiveText() const = 0;
	
	/** Event invoked when the visibility of the widget should change. */
	virtual FSimpleMulticastDelegate& OnVisibilityChanged() = 0;
	
	virtual ~IExternalSandboxActiveViewModel() = default;
};
}
