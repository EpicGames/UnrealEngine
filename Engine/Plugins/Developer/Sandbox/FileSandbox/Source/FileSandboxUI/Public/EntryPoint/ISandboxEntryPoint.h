// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FString;
class FText;
namespace UE::FileSandboxCore { class ISandboxInstance; }

namespace UE::FileSandboxUI
{
/** Implemented by editor systems that makes use of the sandbox feature. */
class ISandboxEntryPoint
{
public:
	
	/** Summons the UI in which the sandboxes  */
	virtual void SummonProviderUI() = 0;
	
	/**
	 * @return The name of the UI that will be summoned. This is displayed e.g. in a summon button. 
	 * This should be an app name, like "Multi-User" or "Sandboxed Editing".
	 */
	virtual FText GetEntryPointLabel() const = 0;
	
	/** @return Whether this provider owns the active sandbox instance. */
	virtual bool OwnsSandbox(const FileSandboxCore::ISandboxInstance& InSandbox) const = 0;
	
	virtual ~ISandboxEntryPoint() = default;
};
}
