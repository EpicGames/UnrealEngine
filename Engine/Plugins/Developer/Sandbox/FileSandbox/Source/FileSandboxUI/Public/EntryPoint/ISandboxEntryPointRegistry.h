// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointerFwd.h"

class FString;

namespace UE::FileSandboxUI
{
class ISandboxEntryPoint;

/** Knows about editor systems that start sandboxes. */
class ISandboxEntryPointRegistry
{
public:
	
	/** Registers the entry point. */
	virtual void RegisterEntryPoint(const TSharedRef<ISandboxEntryPoint>& InEntryPoint) = 0;
	/** Unregisters the entry point. */
	virtual void UnregisterEntryPoint(const TSharedRef<ISandboxEntryPoint>& InEntryPoint) = 0;
	
	/** Finds the entry point that owns the active sandbox. */
	virtual TSharedPtr<ISandboxEntryPoint> FindOwnerOfActiveSandbox() const = 0;
	
	/** Event invoked when the registered entry points have changed. */
	virtual FSimpleMulticastDelegate& OnEntryPointsChanged() = 0;
	
	virtual ~ISandboxEntryPointRegistry() = default;
};
}
