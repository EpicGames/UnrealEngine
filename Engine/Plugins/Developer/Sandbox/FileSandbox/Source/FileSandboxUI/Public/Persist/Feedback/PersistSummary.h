// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

namespace UE::FileSandboxUI
{
/** Summary of a persist operation. */
struct FPersistSummary
{
	/** Number of edited files. */
	int32 Edited = 0;
	/** Number of added files. */
	int32 Added = 0;
	/** Number of deletes files. */
	int32 Deleted = 0;
	
	/** Non-sandbox paths of files that were not persisted. */
	TArray<FString> FailedFiles;
};
}