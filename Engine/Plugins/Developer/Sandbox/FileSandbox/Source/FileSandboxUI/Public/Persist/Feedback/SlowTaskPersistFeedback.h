// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interface/IPersistFeedback.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"

namespace UE::FileSandboxUI
{
DECLARE_DELEGATE_RetVal_OneParam(FText, FFormatPathDelegate, const TCHAR* InFileName);

/**
 * Starts a slow task for the duration of the persist operation. 
 * Remember that nested of FScopedSlowTask works.
 */
class FSlowTaskPersistFeedback : public FileSandboxCore::IPersistFeedback
{
public:
	
	/** Spawns the progress bar. */
	FScopedSlowTask SlowTask;
	
	/** Optional delegate that produces the progress bar text. */
	FFormatPathDelegate FormatPathDelegate; 
	
	explicit FSlowTaskPersistFeedback(int32 InNumFilesToPersist, FFormatPathDelegate InFormatPath = FFormatPathDelegate())
		: SlowTask(InNumFilesToPersist)
		, FormatPathDelegate(InFormatPath)
	{}

	//~ Begin IPersistFeedback Interface
	virtual void StartFile(const TCHAR* InFilename) override
	{
		const FText Text = FormatPathDelegate.IsBound() 
			? FormatPathDelegate.Execute(InFilename)
			: FText::Format(
				NSLOCTEXT("FProgressBarPersistFeedback", "PersistFileFmt", "Persisting {0}"),
				FText::AsCultureInvariant(FPaths::GetBaseFilename(InFilename))
				);
		SlowTask.EnterProgressFrame(1, Text);
	}
	virtual void HandleSuccess(const TCHAR* InFilename) override {}
	virtual void HandleError_CheckoutNotAllowed(const TCHAR* InFilename) override {}
	virtual void HandleError_Checkout(const TCHAR* InFilename) override {}
	virtual void HandleError_Revert(const TCHAR* InFilename) override {}
	virtual void HandleError_MarkForAdd(const TCHAR* InFilename) override {}
	virtual void HandleError_DeleteSCC(const TCHAR* InFilename) override {}
	virtual void HandleError_MakeWritable(const TCHAR* InFilename) override {}
	virtual void HandleError_MoveFile(const TCHAR* InToFilename, const TCHAR* InFromFilename) override {}
	virtual void HandleError_DeleteFile(const TCHAR* InFilename) override {}
	//~ End IPersistFeedback Interface
};
}
