// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "PersistSummary.h"
#include "Interface/IPersistFeedback.h"
#include "Types/GatheredFileChanges.h"

#define UE_API FILESANDBOXUI_API

namespace UE::FileSandboxCore { enum class ESandboxFileChange : uint8; }

namespace UE::FileSandboxUI
{
/**
 * Fills in a FPersistSummary so you
 */
class FSummaryPersistFeedback : public FileSandboxCore::IPersistFeedback
{
	TMap<FString, FileSandboxCore::ESandboxFileChange> FileActions;
	
public:
	
	/** The summary of the operation. */
	FPersistSummary Summary;
	
	UE_API explicit FSummaryPersistFeedback(
		TConstArrayView<FString> InPersistedFiles,
		TConstArrayView<FileSandboxCore::ESandboxFileChange> InFileActions
		);
	
	UE_API explicit FSummaryPersistFeedback(const FileSandboxCore::FGatheredFileChanges& InChanges)
		: FSummaryPersistFeedback(InChanges.NonSandboxPaths, InChanges.FileActions)
	{}

	//~ Begin IPersistFeedback Interface
	virtual void StartFile(const TCHAR* InFilename) override { }
	UE_API virtual void HandleSuccess(const TCHAR* InFilename) override;
	virtual void HandleError_CheckoutNotAllowed(const TCHAR* InFilename) override { HandleError(InFilename); }
	virtual void HandleError_Checkout(const TCHAR* InFilename) override { HandleError(InFilename); }
	virtual void HandleError_Revert(const TCHAR* InFilename) override { HandleError(InFilename); }
	virtual void HandleError_MarkForAdd(const TCHAR* InFilename) override { HandleError(InFilename); }
	virtual void HandleError_DeleteSCC(const TCHAR* InFilename) override { HandleError(InFilename); }
	virtual void HandleError_MakeWritable(const TCHAR* InFilename) override { HandleError(InFilename); }
	virtual void HandleError_MoveFile(const TCHAR* InToFilename, const TCHAR* InFromFilename) override { HandleError(InToFilename); }
	virtual void HandleError_DeleteFile(const TCHAR* InFilename) override { HandleError(InFilename); }
	//~ End IPersistFeedback Interface
	
private:
	
	UE_API void HandleError(const TCHAR* InToFilename);
};
}

#undef UE_API